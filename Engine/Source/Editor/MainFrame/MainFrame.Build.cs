// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MainFrame : ModuleRules
{
	public MainFrame(ReadOnlyTargetRules Target) : base(Target)
	{

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Documentation",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Engine",
				"EngineSettings",
				"HTTP",
				"InputCore",
				"RHI",
				"RenderCore",
				"Slate",
				"SlateCore",
				"SourceControl",
				"SourceControlWindows",
				"TargetPlatform",
				"DesktopPlatform",
				"EditorFramework",
				"UnrealEd",
				"WorkspaceMenuStructure",
				"MessageLog",
				"UATHelper",
				"TranslationEditor",
				"Projects",
				"DeviceProfileEditor",
				"UndoHistoryEditor",
				"Analytics",
				"ToolMenus",
				"LauncherServices",
				"InterchangeCore",
				"InterchangeEngine",
				"ToolWidgets",
				"WebBrowser",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"LauncherPlatform",
				"GameProjectGeneration",
				"ProjectTargetPlatformEditor",
				"LevelEditor",
				"Settings",
				"SourceCodeAccess",
				"HotReload",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"GameProjectGeneration",
				"ProjectTargetPlatformEditor",
				"LevelEditor",
				"SourceCodeAccess",
				"HotReload",
				"LocalizationDashboard", // not required but causes circular depends issues on Linux/Mac
			}
		);
	}
}
