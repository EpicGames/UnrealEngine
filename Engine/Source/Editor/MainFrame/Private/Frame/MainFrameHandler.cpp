// Copyright Epic Games, Inc. All Rights Reserved.

#include "Frame/MainFrameHandler.h"
#include "HAL/FileManager.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Frame/RootWindowLocation.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Docking/SDockTab.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Subsystems/AssetEditorSubsystem.h"

static bool ShowRestoreAssetsPromptOnStartup = true;
FAutoConsoleVariableRef CVarShowRestoreAssetsPromptOnStartup(TEXT("Mainframe.ShowRestoreAssetsPromptOnStartup"), ShowRestoreAssetsPromptOnStartup, TEXT(""), ECVF_ReadOnly);

static bool ShowRestoreAssetsPromptInPIE = false;
FAutoConsoleVariableRef CVarShowRestoreAssetsPromptInPIE(TEXT("Mainframe.ShowRestoreAssetsPromptInPIE"), ShowRestoreAssetsPromptInPIE,
	TEXT("Restore asset windows when running with PIE at startup (default: false). This doesn't work with immersive mode or if Mainframe.ShowRestoreAssetsPromptOnStartup is set to false."));

void FMainFrameHandler::ShowMainFrameWindow(TSharedRef<SWindow> Window, const bool bStartImmersive, const bool bStartPIE) const
{
	// Make sure viewport windows are maximized/immersed if they need to be
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked< FLevelEditorModule >( TEXT( "LevelEditor" ) );

	if( bStartPIE )
	{
		// Kick off an immersive PIE session immediately!

		if( bStartImmersive )
		{
			// When in immersive play in editor, toggle game view on the active viewport
			const bool bForceGameView = true;

			// Start level viewport initially in immersive mode
			LevelEditor.GoImmersiveWithActiveLevelViewport( bForceGameView );
		}

		LevelEditor.StartPlayInEditorSession();
		Window->ShowWindow();
		// Ensure the window is at the front or else we could end up capturing and locking the mouse to a window that isn't visible
		bool bForceWindowToFront = true;
		Window->BringToFront( bForceWindowToFront );

		// Need to register after the window is shown or else we cant capture the mouse
		TSharedPtr<IAssetViewport> Viewport = LevelEditor.GetFirstActiveViewport();
		Viewport->RegisterGameViewportIfPIE();

		if (!bStartImmersive)
		{
			// Command lines may not have processed yet, flush them here to make sure cvars are up to date.
			GEngine->TickDeferredCommands();

			if (ShowRestoreAssetsPromptInPIE && ShowRestoreAssetsPromptOnStartup)
			{
				// Restore any assets we had open. Can be requested to function in PIE
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->RequestRestorePreviouslyOpenAssets();
			}
		}
	}
	else
	{
		if( bStartImmersive )
		{
			// When in immersive play in editor, toggle game view on the active viewport
			const bool bForceGameView = true;

			// Start level viewport initially in immersive mode
			LevelEditor.GoImmersiveWithActiveLevelViewport( bForceGameView );
		}

		// Show the window!
		Window->ShowWindow();

		if( bStartImmersive )
		{
			// Ensure the window is at the front or else we could end up capturing and locking the mouse to a window that isn't visible
			bool bForceWindowToFront = true;
			Window->BringToFront( bForceWindowToFront );
		}
		else
		{
			// Focus the level editor viewport
			LevelEditor.FocusViewport();

			if(ShowRestoreAssetsPromptOnStartup)
			{
				// Restore any assets we had open. Note we don't do this on immersive PIE as its annoying to the user.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->RequestRestorePreviouslyOpenAssets();
			}
		}
	}
}

void FMainFrameHandler::ShutDownEditor()
{
	FEditorDelegates::OnShutdownPostPackagesSaved.Broadcast();

	// By this point we've opted to discard anything we didn't want to save, so disable the auto-save restore.
	GUnrealEd->GetPackageAutoSaver().UpdateRestoreFile(false);
	// Any pending autosaves should not happen.  A tick will go by before the editor shuts down and we want to avoid auto-saving during this time.
	GUnrealEd->GetPackageAutoSaver().ResetAutoSaveTimer();

	GEditor->RequestEndPlayMap();

	// End any play on console/PC games still happening
	GEditor->EndPlayOnLocalPc();

	// Cancel any current Launch On in progress
	GEditor->CancelPlayingViaLauncher();
	
	//Broadcast we are closing the editor
	GEditor->BroadcastEditorClose();

	TSharedPtr<SWindow> RootWindow = RootWindowPtr.Pin();

	// Save root window placement so we can restore it.
	bool bRenderOffScreen = FSlateApplication::Get().IsRenderingOffScreen();
	if (!GUsingNullRHI && !bRenderOffScreen && RootWindow.IsValid())
	{
		FSlateRect WindowRect = RootWindow->GetNonMaximizedRectInScreen();

		if (!RootWindow->HasOSWindowBorder())
		{
			// If the window has a specified border size, shrink its screen size by that amount to prevent it from growing
			// over multiple shutdowns
			const FMargin WindowBorder = RootWindow->GetNonMaximizedWindowBorderSize();
			WindowRect.Right -= WindowBorder.Left + WindowBorder.Right;
			WindowRect.Bottom -= WindowBorder.Top + WindowBorder.Bottom;
		}

		// Save without any DPI Scale so we can save the position and scale in a DPI independent way
		const float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowRect.Left, WindowRect.Top);

		FRootWindowLocation RootWindowLocation(FVector2D(WindowRect.Left, WindowRect.Top)/ DPIScale, WindowRect.GetSize() / DPIScale, RootWindow->IsWindowMaximized());
		RootWindowLocation.SaveToIni();
	}

	// Clear the callback for destructionfrom the main tab; otherwise it will re-enter this shutdown function.
	if (MainTabPtr.IsValid())
	{
		MainTabPtr.Pin()->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
	}

	if (RootWindow.IsValid())
	{
		RootWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride());
		RootWindow->RequestDestroyWindow();
	}

	// Save out any config settings for the editor so they don't get lost
	GEditor->SaveConfig();
	GLevelEditorModeTools().SaveConfig();

	// Delete user settings, if requested
	if (FUnrealEdMisc::Get().IsDeletePreferences())
	{
		IFileManager::Get().Delete(*GEditorPerProjectIni);
	}

	// Take a screenshot of this project for the project browser
	if (FApp::HasProjectName())
	{
		const FString ExistingBaseFilename = FString(FApp::GetProjectName()) + TEXT(".png");
		const FString ExistingScreenshotFilename = FPaths::Combine(*FPaths::ProjectDir(), *ExistingBaseFilename);

		// If there is already a screenshot, no need to take an auto screenshot
		if (!FPaths::FileExists(ExistingScreenshotFilename))
		{
			const FString ScreenShotFilename = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("AutoScreenshot.png"));
			FViewport* Viewport = GEditor->GetActiveViewport();
			if (Viewport)
			{
				UThumbnailManager::CaptureProjectThumbnail(Viewport, ScreenShotFilename, false);
			}
		}
	}

	// Shut down the editor
	// NOTE: We can't close the editor from within this stack frame as it will cause various DLLs
	//       (such as MainFrame) to become unloaded out from underneath the code pointer.  We'll shut down
	//       as soon as it's safe to do so.
	// Note this is the only place in slate that should be calling QUIT_EDITOR
	GEngine->DeferredCommands.Add(TEXT("QUIT_EDITOR"));
}

void FMainFrameHandler::EnableTabClosedDelegate()
{
	if (MainTabPtr.IsValid())
	{
		MainTabPtr.Pin()->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FMainFrameHandler::ShutDownEditor));
		MainTabPtr.Pin()->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &FMainFrameHandler::CanCloseTab));
	}
}

void FMainFrameHandler::DisableTabClosedDelegate()
{
	if (MainTabPtr.IsValid())
	{
		MainTabPtr.Pin()->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
		MainTabPtr.Pin()->SetCanCloseTab(SDockTab::FCanCloseTab());
	}
}
