// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "UnrealEdMisc.h"
#include "Misc/App.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutService.h"
#include "EngineGlobals.h"
#include "InterchangeManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "MainFrameLog.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Misc/MessageDialog.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "MainFrameHandler"

const FText StaticGetApplicationTitle( const bool bIncludeGameName );

/**
 * Helper class that handles talking to the Slate App Manager.
 */
class FMainFrameHandler : public TSharedFromThis<FMainFrameHandler>
{
public:

	/**
	 * Shuts down the Editor.
	 */
	void ShutDownEditor( TSharedRef<SDockTab> TabBeingClosed )
	{
		ShutDownEditor();
	}

	/**
	 * Shuts down the Editor.
	 */
	void ShutDownEditor( );
	
	/**
	 * Checks whether the main frame tab can be closed.
	 *
	 * @return true if the tab can be closed, false otherwise.
	 */
	bool CanCloseTab()
	{
		if ( IsEngineExitRequested() )
		{
			UE_LOGF(LogMainFrame, Warning, "MainFrame: Shutdown already in progress when CanCloseTab was queried, approve tab for closure.");
			return true;
		}
		return CanCloseEditor();
	}

	/**
	 * Checks whether the Editor tab can be closed.
	 *
	 * @return true if the Editor can be closed, false otherwise.
	 */
	bool CanCloseEditor()
	{
		// We don't want to close the editor if we are recreating the mainframe module
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		if (MainFrameModule.IsRecreatingDefaultMainFrame())
		{
			return false;
		}

		if ( FSlateApplication::IsInitialized() && !FSlateApplication::Get().IsNormalExecution())
		{
			// DEBUGGER EXIT PATH

			// The debugger is running we cannot actually close now.
			// We will stop the debugger and enque a request to close the editor on the next frame.

			// Stop debugging.
			FSlateApplication::Get().LeaveDebuggingMode();

			// Defer the call RequestCloseEditor() till next tick.
			GEngine->DeferredCommands.AddUnique( TEXT("CLOSE_SLATE_MAINFRAME") );

			// Cannot exit right now.
			return false;
		}
		else if (UE::IsSavingPackage() || IsGarbageCollecting() || IsLoading() || GIsSlowTask)
		{
			// Saving / Loading / GC / Slow Task path.  It is all unsafe to do that here.

			// We're currently saving or garbage collecting and can't close the editor just yet.
			// We will have to wait and try to request to close the editor on the next frame.

			// Defer the call RequestCloseEditor() till next tick.
			GEngine->DeferredCommands.AddUnique( TEXT("CLOSE_SLATE_MAINFRAME") );

			// Cannot exit right now.
			return false;
		}
		else
		{
			// NORMAL EXIT PATH

			if (UInterchangeManager::GetInterchangeManager().WarnIfInterchangeIsActive()) { return false; }

			// Unattented mode can always exit.
			if (FApp::IsUnattended())
			{
				return true;
			}
			
			// We can't close if lightmass is currently building
			if (GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning()) {return false;}

			// We can't close if a modal dialog is on screen. (Clicking on the Editor window top right 'X' already prevent that, but closing from the Windows task bar preview does not)
			bool bOkToExit = FSlateApplication::IsInitialized() ? !FSlateApplication::Get().GetActiveModalWindow().IsValid() : true;

			// Check if level Mode is open this does PostEditMove processing on actors when it closes so need to do this first before save dialog
			if( GLevelEditorModeTools().IsModeActive( FBuiltinEditorModes::EM_Level ) || 
				GLevelEditorModeTools().IsModeActive( FBuiltinEditorModes::EM_StreamingLevel) )
			{
				GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_Level);
				GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_StreamingLevel);
				bOkToExit = false;
			}

			// Can we close all the major tabs? They have sub-editors in them that might want to not close
			{
				// Ignore the LevelEditor tab; it invoked this function in the first place.
				TSet< TSharedRef<SDockTab> > TabsToIgnore;
				if ( MainTabPtr.IsValid() )
				{
					TabsToIgnore.Add( MainTabPtr.Pin().ToSharedRef() );
				}

				bOkToExit = bOkToExit && FGlobalTabmanager::Get()->CanCloseManager(TabsToIgnore);
			}
			
			// Allow Plugins and other systems to prevent close 
			bOkToExit = bOkToExit && MainFrameModule.ExecuteCanCloseEditorDelegates();

			// Prompt for save and quit only if we did not launch a gameless rocket exe or are in demo mode or we are asking for a close to recreate the Default Main Frame
			if ( FApp::HasProjectName() && !GIsDemoMode && !MainFrameModule.IsRecreatingDefaultMainFrame())
			{
				// Prompt the user to save packages/maps.
				bool bHadPackagesToSave = false;
				{
					const bool bPromptUserToSave = true;
					const bool bSaveMapPackages = true;
					const bool bSaveContentPackages = true;
					const bool bFastSave = false;
					const bool bNotifyNoPackagesSaved = false;
					const bool bCanBeDeclined = true;
					bOkToExit = bOkToExit && FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, &bHadPackagesToSave);
				}

				// If there were packages to save, or switching project, then the user already had a chance to bail out of exiting.
				const bool bPromptedForExit = bHadPackagesToSave || !FUnrealEdMisc::Get().GetPendingProjectName().IsEmpty();
				if ( !bOkToExit && !bPromptedForExit )
				{
					FUnrealEdMisc::Get().ClearPendingProjectName();
					FGlobalTabmanager::Get()->SetCanSavePersistentLayouts(true);
					FUnrealEdMisc::Get().ForceDeletePreferences(false);
					FUnrealEdMisc::Get().ClearConfigRestoreFilenames();
				}
				else if (bOkToExit && !bPromptedForExit && GetDefault<UEditorPerProjectUserSettings>()->bConfirmEditorClose)
				{
					const EAppReturnType::Type Response = FMessageDialog::Open(
						EAppMsgCategory::Info,
						EAppMsgType::YesNo,
						LOCTEXT("ConfirmClose", "Are you sure you want to close the Unreal Editor?"),
						LOCTEXT("ConfirmCloseTitle", "Close Editor")
					);

					bOkToExit = (Response == EAppReturnType::Yes);
				}
			}
			
			return bOkToExit;
		}
	}

	void CloseRootWindowOverride( const TSharedRef<SWindow>& WindowBeingClosed )
	{
		// Save the visual state of the editor before we even
		// ask whether we can shut down.
		TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
		if (GlobalTabManager->CanSavePersistentLayouts())
		{
			GlobalTabManager->SaveAllVisualState();
		}

		if (CanCloseEditor())
		{
			ShutDownEditor();
		}
	}

	/**
	 * Method that handles the generation of the mainframe, given the window it resides in,
	 * and a string which determines the initial layout of its primary dock area.
	 */
	void OnMainFrameGenerated( const TSharedPtr<SDockTab>& MainTab, const TSharedRef<SWindow>& InRootWindow )
	{
		TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
		
		if (MainTab.IsValid())
		{
			GlobalTabManager->SetMainTab(MainTab.ToSharedRef());
		}

		// Persistent layouts should get stored using the specified method.
		GlobalTabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateRaw(this, &FMainFrameHandler::HandleTabManagerPersistLayout));
		
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));

		const bool bIncludeGameName = true;
		GlobalTabManager->SetApplicationTitle( MainFrameModule.GetApplicationTitle( bIncludeGameName ) );
		
		InRootWindow->SetRequestDestroyWindowOverride( FRequestDestroyWindowOverride::CreateRaw( this, &FMainFrameHandler::CloseRootWindowOverride ) );

		MainTabPtr = MainTab;
		RootWindowPtr = InRootWindow;

		EnableTabClosedDelegate();
	}

	/**
	 * Shows the main frame window.  Call this after you've setup initial layouts to reveal the window 
	 *
	 * @param bStartImmersive True to force a main frame viewport into immersive mode
	 * @param bStartPIE True to start a PIE session right away
	 */
	void ShowMainFrameWindow(TSharedRef<SWindow> Window, const bool bStartImmersive, const bool bStartPIE) const;

	/** Gets the parent window of the mainframe */
	TSharedPtr<SWindow> GetParentWindow() const
	{
		return RootWindowPtr.Pin();
	}

	/** Sets the reference to the main tab */
	void SetMainTab(const TSharedRef<SDockTab>& MainTab)
	{
		MainTabPtr = MainTab;
	}
	
	/** Enables the delegate responsible for shutting down the editor when the main tab is closed */
	void EnableTabClosedDelegate();

	/** Disables the delegate responsible for shutting down the editor when the main tab is closed */
	void DisableTabClosedDelegate();

private:

	// Callback for persisting the Level Editor's layout.
	void HandleTabManagerPersistLayout( const TSharedRef<FTabManager::FLayout>& LayoutToSave )
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	}

private:

	/** Editor main frame window */
	TWeakPtr<SDockTab> MainTabPtr;

	/** The window that all of the editor is parented to. */
	TWeakPtr<SWindow> RootWindowPtr;
};

#undef LOCTEXT_NAMESPACE
