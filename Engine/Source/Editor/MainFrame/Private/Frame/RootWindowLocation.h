// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplication.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/ConfigCacheIni.h"

/**
 * Describes the position and size of the main window.
 */
struct FRootWindowLocation
{
	/**
	 * Holds the window's position on the screen.
	 */
	FVector2D ScreenPosition;

	/**
	 * Holds the size of the window.
	 */
	FVector2D WindowSize;

	/**
	 * Whether the window is initially maximized.
	 */
	bool InitiallyMaximized;


public:

	/**
	 * Default constructor.
	 */
	FRootWindowLocation( )
		: WindowSize( VectorFromSettings(TEXT("WindowSize"), FVector2D(1280,720)) )
		, InitiallyMaximized( BoolFromSettings(TEXT("InitiallyMaximized"), true) )
	{
		ScreenPosition = VectorFromSettings(TEXT("ScreenPosition"), GetCenteredScreenPosition() );
	}
	
	/**
	 * Creates and initializes a new instance with the specified size.
	 */
	FRootWindowLocation(FVector2D InWindowSize, bool InInitiallyMaximized)
		: WindowSize(InWindowSize)
		, InitiallyMaximized(InInitiallyMaximized)
	{
		ScreenPosition = GetCenteredScreenPosition();
	}

	/**
	 * Creates and initializes a new instance with the specified position and size.
	 */
	FRootWindowLocation( FVector2D InScreenPosition, FVector2D InWindowSize, bool InInitiallyMaximized )
		: ScreenPosition( InScreenPosition )
		, WindowSize( InWindowSize )
		, InitiallyMaximized( InInitiallyMaximized )
	{ }


public:

	/**
	 * Set centered screen position based on the size
	 */
	FVector2D GetCenteredScreenPosition() const
	{
		// Find the default centered screen position
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);
		const FVector2D DisplayTopLeft(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left, DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);
		const FVector2D DisplaySize(DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
									DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);
		return DisplayTopLeft + (DisplaySize - WindowSize) * 0.5f;
	}

	/**
	 * Saves this structure to the INI file.
	 */
	void SaveToIni( )
	{
		GConfig->SetString( TEXT("RootWindow"), TEXT("ScreenPosition"), *ScreenPosition.ToString(), GEditorPerProjectIni );
		GConfig->SetString( TEXT("RootWindow"), TEXT("WindowSize"), *WindowSize.ToString(), GEditorPerProjectIni );
		GConfig->SetBool( TEXT("RootWindow"), TEXT("InitiallyMaximized"), InitiallyMaximized, GEditorPerProjectIni );
	}


private:

	static FVector2D VectorFromSettings( const TCHAR* SettingName, FVector2D DefaultValue )
	{
		FVector2D ReturnValue = DefaultValue;
		FString ValueAsString;
		if ( GConfig->GetString(TEXT("RootWindow"), SettingName, ValueAsString, GEditorPerProjectIni) && ReturnValue.InitFromString(ValueAsString) )
		{
			// Successfully loaded setting
			return ReturnValue;
		}
		else
		{
			return DefaultValue;
		}
	}

	static bool BoolFromSettings( const TCHAR* SettingName, bool DefaultValue )
	{
		bool ReturnValue;
		if ( GConfig->GetBool(TEXT("RootWindow"), SettingName, ReturnValue, GEditorPerProjectIni) )
		{
			return ReturnValue;
		}
		else
		{
			return DefaultValue;
		}
	}
};
