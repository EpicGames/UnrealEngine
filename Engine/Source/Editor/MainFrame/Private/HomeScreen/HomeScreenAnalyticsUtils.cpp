// Copyright Epic Games, Inc. All Rights Reserved.

#include "HomeScreenAnalyticsUtils.h"
#include "EngineAnalytics.h"

void FHomeScreenInteractionAnalyticsUtils::RegisterInteractionUserAnalytics(FHomeScreenInteractionAnalyticsParam InParam)
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		// Add the type of the interaction
		static const UEnum* InteractionTypeEnum = StaticEnum<EHomeScreenInteractionType>();
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("InteractionType"), InteractionTypeEnum->GetNameByValue((int64)InParam.InteractionType)));

		// Add if the user is a first timer based on projects found into analytics
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HasProjectsOnMachine"),  InParam.bHasProjectsOnMachine));

		// Add the user current section
		static const UEnum* MainSectionEnum = StaticEnum<EMainSectionMenu>();
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ActiveSection"),  MainSectionEnum->GetNameByValue((int64)InParam.ActiveSection)));

		// Add the user current LoadAtStartup setting
		static const UEnum* LoadAtStartupEnum = StaticEnum<EAutoLoadProject>();
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LoadAtStartup"),  LoadAtStartupEnum->GetNameByValue((int64)InParam.LoadAtStartup)));

		// Add the link requested or none if empty
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DestinationURL"),  InParam.DestinationURL));

		// Add the element the user interacted with
		static const UEnum* HomeScreenElementEnum = StaticEnum<EHomeScreenElement>();
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("InteractedElement"), HomeScreenElementEnum->GetNameByValue((int64)InParam.HomeScreenElement)));

		FEngineAnalytics::GetProvider().RecordEvent("Editor.HomeScreen.Interaction", EventAttributes);
	}
}
