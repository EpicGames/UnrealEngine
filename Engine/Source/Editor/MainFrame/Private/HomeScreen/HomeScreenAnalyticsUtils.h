// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Settings/HomeScreenCommon.h"
#include "HomeScreenAnalyticsUtils.generated.h"

// Enum used for Analytics event for the interacted element
UENUM()
enum class EHomeScreenElement : uint8
{
	None,
	CreateNewProjectButton,
	OpenProjectButton,
	SectionButton,
	ResourceButton,
	LoadAtStartup,
	SocialLinkButton,
	WebLinkButton,
	CreateStartupProjectButton
};

// Enum used for Analytics event for the type of interaction
UENUM()
enum class EHomeScreenInteractionType : uint8
{
	None,
	Click
};

// Helper struct that contains the params for the analytic events
struct FHomeScreenInteractionAnalyticsParam
{
	FHomeScreenInteractionAnalyticsParam()
		: InteractionType(EHomeScreenInteractionType::None)
		, bHasProjectsOnMachine(true)
		, ActiveSection(EMainSectionMenu::None)
		, LoadAtStartup(EAutoLoadProject::MAX)
		, DestinationURL(TEXT("None"))
		, HomeScreenElement(EHomeScreenElement::None)
	{}

	// Type of the interaction, currently only Click is supported
	EHomeScreenInteractionType InteractionType;
	
	// Whether the user has already projects on its machine
	bool bHasProjectsOnMachine;
	
	// The current active section
	EMainSectionMenu ActiveSection;

	// Load at startup option
	EAutoLoadProject LoadAtStartup;

	// Link if applicable
	FString DestinationURL;
	
	// The HomeScreen element that the user interacted with
	EHomeScreenElement HomeScreenElement;
};

// Helper function that sends the Interaction event
class FHomeScreenInteractionAnalyticsUtils
{
public:
	/** Register a new HomeScreen analytics event based on the given parameters */
	static void RegisterInteractionUserAnalytics(FHomeScreenInteractionAnalyticsParam InParam);
};
