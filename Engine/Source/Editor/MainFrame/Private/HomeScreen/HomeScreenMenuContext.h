// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "HomeScreenMenuContext.generated.h"

class SHomeScreen;

UCLASS(MinimalAPI)
class UHomeScreenContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<SHomeScreen> HomeScreen;
};
