// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/HomeScreenSettings.h"
#include "Settings/HomeScreenCommon.h"
#include "Settings/EditorSettings.h"

UHomeScreenSettings::UHomeScreenSettings()
{
	FOnSetLoadAtStartup OnSetLoadAtStartup = FOnSetLoadAtStartup::CreateUObject(this, &UHomeScreenSettings::SetLoadAtStartupChanged);
	FOnGetLoadAtStartup OnGetLoadAtStartup = FOnGetLoadAtStartup::CreateUObject(this, &UHomeScreenSettings::GetLoadAtStartupChanged);
	HomeScreenLoadAtStartupSetting = FHomeScreenLoadAtStartupSetting(OnSetLoadAtStartup, OnGetLoadAtStartup);
}

void UHomeScreenSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	FName NAME_LoadAtStartup = GET_MEMBER_NAME_CHECKED(UHomeScreenSettings, LoadAtStartup);

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == NAME_LoadAtStartup)
		{
			UEditorSettings *Settings = GetMutableDefault<UEditorSettings>();

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Settings->bLoadTheMostRecentlyLoadedProjectAtStartup = LoadAtStartup == EAutoLoadProject::LastProject;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FProperty* AutoloadProjectProperty = FindFProperty<FProperty>(Settings->GetClass(), "bLoadTheMostRecentlyLoadedProjectAtStartup");
			if (AutoloadProjectProperty != nullptr)
			{
				FPropertyChangedEvent PropertyUpdateStruct(AutoloadProjectProperty);
				Settings->PostEditChangeProperty(PropertyUpdateStruct);
			}
			// Do not broadcast it here because the EditorSettings is going to call our OnMostRecentProjectEditorSettingChanged which will broadcast the change from the HomeScreenSettings
		}
	}
}

void UHomeScreenSettings::PostInitProperties()
{
	Super::PostInitProperties();
	if (UEditorSettings *Settings = GetMutableDefault<UEditorSettings>())
	{
		Settings->OnMostRecentProjectSettingChanged().AddUObject(this, &UHomeScreenSettings::OnMostRecentProjectEditorSettingChanged);
	}
}

FHomeScreenLoadAtStartupSetting& UHomeScreenSettings::GetLoadAtStartupSetting()
{
	return HomeScreenLoadAtStartupSetting;
}

void UHomeScreenSettings::OnMostRecentProjectEditorSettingChanged(bool bInAutoLoad)
{
	LoadAtStartup = bInAutoLoad ? EAutoLoadProject::LastProject : EAutoLoadProject::HomeScreen;

	// Broadcast that the setting changed
	HomeScreenLoadAtStartupSetting.OnLoadAtStartupChanged().Broadcast(LoadAtStartup);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnLoadAtStartupChanged().Broadcast(LoadAtStartup);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UHomeScreenSettings::SetLoadAtStartupChanged(bool bInAutoLoad)
{
	EAutoLoadProject NewValue = bInAutoLoad ? EAutoLoadProject::LastProject : EAutoLoadProject::HomeScreen;

	if (LoadAtStartup == NewValue)
	{
		return;
	}

	LoadAtStartup = NewValue;

	// Manually trigger the property changed event so that the home screen editor settings UI updates to reflect the change
	FProperty* LoadAtStartupProperty = FindFProperty<FProperty>(GetClass(), "LoadAtStartup");
	if (LoadAtStartupProperty != NULL)
	{
		FPropertyChangedEvent PropertyUpdateStruct(LoadAtStartupProperty);
		PostEditChangeProperty(PropertyUpdateStruct);
	}
}

bool UHomeScreenSettings::GetLoadAtStartupChanged() const
{
	return LoadAtStartup == EAutoLoadProject::LastProject;
}
