// Copyright Epic Games, Inc. All Rights Reserved.

#include "HomeScreenWeb.h"
#include "Internationalization/Regex.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "HomeScreenWeb"

void UHomeScreenWeb::NavigateTo(EMainSectionMenu InSectionToNavigate)
{
	SectionToNavigate = InSectionToNavigate;

	OnNavigationChangedDelegate.Broadcast(InSectionToNavigate);
}

void UHomeScreenWeb::OpenGettingStartedProject()
{
	OnTutorialProjectRequestedDelegate.Broadcast();
}

void UHomeScreenWeb::OpenWebPage(const FString& InURL) const
{
	const FString ValidURLRegex = TEXT("^https?://");
	const FRegexPattern URLPattern(ValidURLRegex, ERegexPatternFlags::CaseInsensitive);
	FRegexMatcher URLMatcher(URLPattern, InURL);

	if (URLMatcher.FindNext())
	{
		OnOpenLinkDelegate.Broadcast(InURL);
	}
	else
	{
		const FText ErrorText = FText::Format(LOCTEXT("LaunchingURLNotValid", "URL is not valid:\n {0}"), FText::FromString(InURL));
		FMessageDialog::Open(EAppMsgType::Ok, ErrorText);
	}
}

#undef LOCTEXT_NAMESPACE
