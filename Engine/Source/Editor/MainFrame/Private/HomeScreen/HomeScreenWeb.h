// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Settings/HomeScreenCommon.h"
#include "UObject/Object.h"
#include "HomeScreenWeb.generated.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnNavigationChanged, const EMainSectionMenu);
DECLARE_MULTICAST_DELEGATE(FOnGettingStartedProjectRequested);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnOpenLink, FString);

UCLASS()
class UHomeScreenWeb : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION()
	void NavigateTo(EMainSectionMenu InSectionToNavigate);
	FOnNavigationChanged& OnNavigationChanged() { return OnNavigationChangedDelegate; }

	UFUNCTION()
	void OpenGettingStartedProject();
	FOnGettingStartedProjectRequested& OnTutorialProjectRequested() { return OnTutorialProjectRequestedDelegate; }

	UFUNCTION()
	void OpenWebPage(const FString& InURL) const;
	FOnOpenLink& OnOpenLink() { return OnOpenLinkDelegate; }

private:
	FOnNavigationChanged OnNavigationChangedDelegate;
	FOnGettingStartedProjectRequested OnTutorialProjectRequestedDelegate;
	FOnOpenLink OnOpenLinkDelegate;
	EMainSectionMenu SectionToNavigate = EMainSectionMenu::None;
};
