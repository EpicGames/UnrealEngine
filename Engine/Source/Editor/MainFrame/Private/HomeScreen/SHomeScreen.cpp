// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHomeScreen.h"

#include "DesktopPlatformModule.h"
#include "Dialogs/SOutputLogDialog.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Features/IModularFeatures.h"
#include "Frame/MainFrameActions.h"
#include "Frame/RootWindowLocation.h"
#include "GameProjectGenerationModule.h"
#include "GameProjectUtils.h"
#include "HomeScreenMenuContext.h"
#include "Settings/HomeScreenSettings.h"
#include "HomeScreenWeb.h"
#include "HttpModule.h"
#include "HttpRetrySystem.h"
#include "IWebBrowserSingleton.h"
#include "IWebBrowserWindow.h"
#include "MainFrameModule.h"
#include "ProjectDescriptor.h"
#include "Settings/EditorSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "SWebBrowser.h"
#include "TimerManager.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "WebBrowserModule.h"
#include "Internationalization/Culture.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "SHomeScreen"

SHomeScreen::~SHomeScreen()
{
	StopInternetConnectionTickCheck();

	if (WebObject.IsValid())
	{
		WebObject->OnNavigationChanged().RemoveAll(this);
		WebObject->OnTutorialProjectRequested().RemoveAll(this);
		WebObject->OnOpenLink().RemoveAll(this);
	}

	if (HttpRetryRequest.IsValid())
	{
		HttpRetryRequest->OnProcessRequestComplete().Unbind();
		HttpRetryRequest.Reset();
	}

	FGlobalTabmanager::Get()->OnTabForegrounded_Unsubscribe(OnTabForegroundedHandle);

	HttpRetryManager.Reset();

	// Register a Destroyed event for the analytics
	RegisterOnDestructedAnalyticEvent();
}

void SHomeScreen::Construct(const FArguments& InArgs)
{
	SectionLinks = InArgs._SectionLinks;
	OnGetStartupSettingDelegate = InArgs._OnGetStartupSetting;

	HttpRetryManager = MakeShared<FHttpRetrySystem::FManager>(
		FHttpRetrySystem::FRetryLimitCountSetting(3),
		FHttpRetrySystem::FRetryTimeoutRelativeSecondsSetting(3));

	OnTabForegroundedHandle = FGlobalTabmanager::Get()->OnTabForegrounded_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &SHomeScreen::OnTabForegrounded));

	// Automatically start the internet connection check as if the HomeScreen is not a tab it won't ever start
	StartInternetConnectionTickCheck();

	AutoLoadProjectComboBoxSelection = EAutoLoadProject::HomeScreen;
	if (OnGetStartupSettingDelegate.IsBound())
	{
		FHomeScreenLoadAtStartupSetting& LoadAtStartupSetting = OnGetStartupSettingDelegate.Execute();
		LoadAtStartupSetting.OnLoadAtStartupChanged().AddSP(this, &SHomeScreen::OnLoadAtStartupSettingChanged);
		AutoLoadProjectComboBoxSelection = LoadAtStartupSetting.GetOnLoadAtStartup() ? EAutoLoadProject::LastProject : EAutoLoadProject::HomeScreen;
	}

	bHasProjectsOnMachine = InArgs._AlwaysShowRecentList || HasAlreadyLatestEngineProject();
	MainHomeSelection = bHasProjectsOnMachine ? EMainSectionMenu::Home : EMainSectionMenu::GettingStarted;

	// If the default home section is not found set it to none, so that at least the recent projects are shown
	if (!SectionLinks.Contains(MainHomeSelection))
	{
		MainHomeSelection = EMainSectionMenu::None;
	}

	// Fixed Button Size
	constexpr float SectionButtonCheckBoxHeight = 28.f;

	// Main Section CheckBoxes
	CreateMainSectionCheckBox(HomeCheckBox, EMainSectionMenu::Home, LOCTEXT("ProjectHomeHome", "Home"), FAppStyle::GetBrush("HomeScreen.Home"));
	CreateMainSectionCheckBox(NewsCheckBox, EMainSectionMenu::News, LOCTEXT("ProjectHomeNews", "News"), FAppStyle::GetBrush("HomeScreen.News"));
	CreateMainSectionCheckBox(GettingStartedCheckBox, EMainSectionMenu::GettingStarted, LOCTEXT("ProjectHomeGettingStarted", "Getting Started"), FAppStyle::GetBrush("HomeScreen.Rocket"));
	CreateMainSectionCheckBox(SampleProjectsCheckBox, EMainSectionMenu::SampleProjects, LOCTEXT("ProjectHomeSampleProjects", "Sample Projects"), FAppStyle::GetBrush("HomeScreen.Archive"));

	// Social Media Buttons
	TSharedPtr<SButton> FacebookButton;
	TSharedPtr<SButton> TwitterButton;
	TSharedPtr<SButton> YouTubeButton;
	TSharedPtr<SButton> TwitchButton;
	TSharedPtr<SButton> InstagramButton;

	CreateSocialMediaButtons(FacebookButton, TEXT("https://www.facebook.com/unrealengine"), FAppStyle::GetBrush("SocialMedia.Facebook"));
	CreateSocialMediaButtons(TwitterButton, TEXT("https://twitter.com/unrealengine"), FAppStyle::GetBrush("SocialMedia.Twitter"));
	CreateSocialMediaButtons(YouTubeButton, TEXT("https://www.youtube.com/c/unrealengine"), FAppStyle::GetBrush("SocialMedia.YouTube"));
	CreateSocialMediaButtons(TwitchButton, TEXT("https://www.twitch.tv/unrealengine"), FAppStyle::GetBrush("SocialMedia.Twitch"));
	CreateSocialMediaButtons(InstagramButton, TEXT("https://www.instagram.com/unrealengine/"), FAppStyle::GetBrush("SocialMedia.Instagram"));

	// WebBrowser (delay-loaded)
	SAssignNew(WebBrowserContainer, SBox);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
		.BorderBackgroundColor(COLOR("#101014FF"))
		.Padding(0.f)
		[
			SNew(SHorizontalBox)

			// New Project Section
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(8.f)
				[
					// Title Icon
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					.Padding(0.f, 32.f)
					[
						SNew(SImage)
						.Image(InArgs._UnrealLogoBrush)
					]

					// Create and MyFolder Buttons
					+ SVerticalBox::Slot()
					.AutoHeight()
					.MinHeight(SectionButtonCheckBoxHeight)
					.MaxHeight(SectionButtonCheckBoxHeight)
					.Padding(8.f, 0.f, 8.f, 8.f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HomeScreen.CreateNewProjectButton")
						.OnClicked(this, &SHomeScreen::OnCreateProjectDialog, /** bInAllowProjectOpening */false, /** bInAllowProjectCreation */true)
						.HAlign(HAlign_Center)
						.ContentPadding(0.f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.f, 0.f, 8.f, 0.f)
							[
									SNew(SImage)
									.Image(FAppStyle::GetBrush("HomeScreen.PlusCircle"))
									.ColorAndOpacity(FStyleColors::Black)
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("HomeScreen.Font.NewAndCreateButton"))
								.Text(LOCTEXT("ProjectHomeCreateNewProject", "New Project"))
							]
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.MinHeight(SectionButtonCheckBoxHeight)
					.MaxHeight(SectionButtonCheckBoxHeight)
					.Padding(8.f, 0.f, 8.f, 0.f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HomeScreen.MyFolderButton")
						.OnClicked(this, &SHomeScreen::OnCreateProjectDialog, /** bInAllowProjectOpening */true, /** bInAllowProjectCreation */false)
						.HAlign(HAlign_Center)
						.ContentPadding(0.f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.f, 0.f, 8.f, 0.f)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("HomeScreen.FolderOpen"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("HomeScreen.Font.NewAndCreateButton"))
								.Text(LOCTEXT("ProjectHomeMyProjects", "My Projects"))
							]
						]
					]

					// Main Home Section
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.f, 16.f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.MinHeight(SectionButtonCheckBoxHeight)
						.MaxHeight(SectionButtonCheckBoxHeight)
						[
							HomeCheckBox.ToSharedRef()
						]
					
						+ SVerticalBox::Slot()
						.AutoHeight()
						.MinHeight(SectionButtonCheckBoxHeight)
						.MaxHeight(SectionButtonCheckBoxHeight)
						[
							NewsCheckBox.ToSharedRef()
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.MinHeight(SectionButtonCheckBoxHeight)
						.MaxHeight(SectionButtonCheckBoxHeight)
						[
							GettingStartedCheckBox.ToSharedRef()
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.MinHeight(SectionButtonCheckBoxHeight)
						.MaxHeight(SectionButtonCheckBoxHeight)
						[
							SampleProjectsCheckBox.ToSharedRef()
						]
					]

					// Documentation and Community Section
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.f, 0.f, 8.f, 4.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ResourceAndCommunity", "Resources & Community"))
						.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultBoldSize"))
						.ColorAndOpacity(FStyleColors::White)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(16.f, 0.f)
					[
						SAssignNew(ResourceAndCommunityVerticalBox, SVerticalBox)
					]

					// Load on Startup Section
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Bottom)
					.FillHeight(1.f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.Padding(16.f, 0.f, 16.f, 16.f)
						[
							SNew(SBorder)
							.Clipping(EWidgetClipping::ClipToBounds)
							.BorderImage(FAppStyle::GetBrush("HomeScreen.NoInternet.Border"))
							.Padding(16.f)
							.Visibility_Lambda([this]()
							{
								return bIsConnected ? EVisibility::Collapsed : EVisibility::Visible;
							})
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.f, 0.f, 0.f, 16.f)
								.HAlign(HAlign_Left)
								[
									SNew(SBox)
									.WidthOverride(24.f)
									.HeightOverride(24.f)
									[
										SNew(SWidgetSwitcher)
										.WidgetIndex(this, &SHomeScreen::GetNoInternetIconIndex)

										+ SWidgetSwitcher::Slot()
										[
											SNew(SImage)
											.Image(FAppStyle::GetBrush("HomeScreen.NoInternet.Icon"))
										]

										+ SWidgetSwitcher::Slot()
										[
											SNew(SCircularThrobber)
											.NumPieces(8)
											.Period(1.2f)
											.Radius(12.f)
										]
									]
								]

								+ SVerticalBox::Slot()
								.FillHeight(0.5f)
								.Padding(0.f, 0.f, 0.f, 16.f)
								[
									SNew(STextBlock)
									.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultBoldSize"))
									.Text(LOCTEXT("HomeScreenNoInternet", "No Internet Connection"))
									.ColorAndOpacity(FStyleColors::White)
								]

								+ SVerticalBox::Slot()
								.FillHeight(1.f)
								[
									SNew(SButton)
									.HAlign(HAlign_Center)
									.ButtonStyle(FAppStyle::Get(), "HomeScreen.MyFolderButton")
									.OnClicked(this, &SHomeScreen::OnInternetConnectionRetried)
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
										.Text(LOCTEXT("HomeScreenReconnect", "Reconnect"))
									]
								]
							]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(16.f, 0.f)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Visibility(OnGetStartupSettingDelegate.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
							.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
							.Text(LOCTEXT("ProjectHomeLoad", "Load on Startup"))
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(16.f, 8.f, 16.f, 36.f)
						[
							SAssignNew(AutoLoadProjectComboBox, SComboButton)
							.Visibility(OnGetStartupSettingDelegate.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
							.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("HomeScreen.ComboButton"))
							.MenuPlacement(EMenuPlacement::MenuPlacement_CenteredAboveAnchor)
							.HasDownArrow(true)
							.ContentPadding(FMargin(0.f ,2.f))
							.ButtonContent()
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
								.Text(this, &SHomeScreen::GetAutoLoadProjectComboBoxLabelText)
								.ColorAndOpacity_Lambda([this] ()
								{
									if (AutoLoadProjectComboBox.IsValid() && AutoLoadProjectComboBox->IsHovered())
									{
										return FStyleColors::White;
									}
									return FStyleColors::Foreground;
								})
							]
							.MenuContent()
							[
								CreateComboButtonMenuContentWidget()
							]
						]

						// Social Media Section
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(46.f, 0.f, 46.f, 20.f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.f, 0.f)
							[
								FacebookButton.ToSharedRef()
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.f, 0.f)
							[
								TwitterButton.ToSharedRef()
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.f, 0.f)
							[
								YouTubeButton.ToSharedRef()
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.f, 0.f)
							[
								TwitchButton.ToSharedRef()
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.f, 0.f)
							[
								InstagramButton.ToSharedRef()
							]
						]
					]
				]
			]

			// Web Api
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([this] ()
				{
					// If the main home selection is none or if you are in the Home section and projects were found in your machine show the RecentProjects
					return MainHomeSelection == EMainSectionMenu::None || (MainHomeSelection == EMainSectionMenu::Home && bHasProjectsOnMachine) ? /** Show Recent Projects */ 0 : /** Show just the WebPage */ 1;
				})

				+ SWidgetSwitcher::Slot()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.Padding(0.f, 0.f, 0.f, 24.f)
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.FillWidth(0.125f)
						[
							SNew(SSpacer)
						]

						+ SHorizontalBox::Slot()
						.FillWidth(0.75f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.Padding(0.f, 64.f, 0.f, 48.f)
							[
								SNew(SHorizontalBox)
							
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(0.f, 0.f, 8.f, 0.f)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("RecentProjects", "Recent Projects"))
									.Font(FAppStyle::GetFontStyle("HomeScreen.Font.RecentProject"))
									.ColorAndOpacity(FStyleColors::White)
								]

								+ SHorizontalBox::Slot()
								[
									SNew(SSpacer)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Right)
								[
									SNew(SButton)
									.ButtonStyle(FAppStyle::Get(), "HomeScreen.SeeAllProjectsButton")
									.OnClicked(this, &SHomeScreen::OnCreateProjectDialog, /** bInAllowProjectOpening */true, /** bInAllowProjectCreation */false)
									.ContentPadding(FMargin(16, 8))
									[
										SNew(STextBlock)
										.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultBoldSize"))
										.Text(LOCTEXT("SeeAllProjects", "See all projects"))
									]
								]
							]

							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SBorder)
								.Clipping(EWidgetClipping::ClipToBounds)
								.BorderImage(FAppStyle::GetBrush("HomeScreen.RecentProjects.Background"))
								.Padding(24.f)
								[
									InArgs._RecentProjectList
								]
							]
						]

						+ SHorizontalBox::Slot()
						.FillWidth(0.125f)
						[
							SNew(SSpacer)
						]
					]

					+ SVerticalBox::Slot()
					.FillHeight(1.f)
					[
						WebBrowserContainer.ToSharedRef()
					]
				]

				+ SWidgetSwitcher::Slot()
				[
					WebBrowserContainer.ToSharedRef()
				]
			]
		]
	];

	for (const FHomeScreenResourceCommunityEntry& ResourceCommunityEntries : InArgs._ResourceCommunityEntries)
	{
		TSharedPtr<SButton> OutButton;
		CreateResourceButtons(OutButton, ResourceCommunityEntries.Link, ResourceCommunityEntries.DisplayName, FAppStyle::GetBrush(ResourceCommunityEntries.IconBrushName));
		ResourceAndCommunityVerticalBox->AddSlot()
			.AutoHeight()
			.MinHeight(SectionButtonCheckBoxHeight)
			.MaxHeight(SectionButtonCheckBoxHeight)
			[
				OutButton.ToSharedRef()
			];
	}
}

void SHomeScreen::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CreateWebBrowserWidgetIfNeeded();

	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);
}

void SHomeScreen::OnTabForegrounded(TSharedPtr<SDockTab> NewForegroundTab, TSharedPtr<SDockTab> BackgroundTab)
{
	// Stop the internet connection check if the foreground tab is not valid and return early
	if (!NewForegroundTab.IsValid())
	{
		StopInternetConnectionTickCheck();
		return;
	}

	// If the foreground tab contain this widget start its internet connection timer
	// If it goes to the background then stop the internet connection
	if (NewForegroundTab->GetContent() == AsShared())
	{
		StartInternetConnectionTickCheck();
	}
	else
	{
		StopInternetConnectionTickCheck();
	}
}

FReply SHomeScreen::OnCreateProjectDialog(bool bInAllowProjectOpening, bool bInAllowProjectCreation)
{
	FHomeScreenInteractionAnalyticsParam AnalyticsParam;

	AnalyticsParam.HomeScreenElement = bInAllowProjectCreation ? 
		EHomeScreenElement::CreateNewProjectButton 
		: EHomeScreenElement::OpenProjectButton;

	AnalyticsParam.InteractionType = EHomeScreenInteractionType::Click;
	AnalyticsParam.LoadAtStartup = AutoLoadProjectComboBoxSelection;
	AnalyticsParam.ActiveSection = MainHomeSelection;
	AnalyticsParam.bHasProjectsOnMachine = bHasProjectsOnMachine;

	FHomeScreenInteractionAnalyticsUtils::RegisterInteractionUserAnalytics(AnalyticsParam);

	IMainFrameModule::Get().ShowProjectBrowser(bInAllowProjectOpening, bInAllowProjectCreation);

	return FReply::Handled();
}

FReply SHomeScreen::OnLinkClicked(TAttribute<FString> InURL, const EHomeScreenElement InElement) const
{
	const FString URL = InURL.Get();
	if (!URL.IsEmpty())
	{
		return OnLinkClicked(URL, InElement);
	}
	return FReply::Handled();
}

FReply SHomeScreen::OnLinkClicked(FString InURL, const EHomeScreenElement InElement) const
{
	FHomeScreenInteractionAnalyticsParam AnalyticsParam;
	AnalyticsParam.HomeScreenElement = InElement;
	AnalyticsParam.InteractionType = EHomeScreenInteractionType::Click;
	AnalyticsParam.LoadAtStartup = AutoLoadProjectComboBoxSelection;
	AnalyticsParam.ActiveSection = MainHomeSelection;
	AnalyticsParam.bHasProjectsOnMachine = bHasProjectsOnMachine;
	AnalyticsParam.DestinationURL = InURL;
	FHomeScreenInteractionAnalyticsUtils::RegisterInteractionUserAnalytics(AnalyticsParam);

	FPlatformProcess::LaunchURL(*InURL, nullptr, nullptr);
	return FReply::Handled();
}

void SHomeScreen::OnWebLinkClicked(FString InURL) const
{
	OnLinkClicked(InURL, EHomeScreenElement::WebLinkButton);
}

ECheckBoxState SHomeScreen::IsMainHomeSectionChecked(EMainSectionMenu InMainHomeSelection) const
{
	return MainHomeSelection == InMainHomeSelection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SHomeScreen::OnMainHomeSectionChanged(ECheckBoxState InCheckBoxState, EMainSectionMenu InMainHomeSelection, bool bSkipAnalytics)
{
	// If this HomeScreen version do not have a link for the requested section menu do not process it
	if (!SectionLinks.Contains(InMainHomeSelection))
	{
		return;
	}

	MainHomeSelection = InMainHomeSelection;

	if (!bSkipAnalytics)
	{
		FHomeScreenInteractionAnalyticsParam AnalyticsParam;
		AnalyticsParam.HomeScreenElement = EHomeScreenElement::SectionButton;
		AnalyticsParam.InteractionType = EHomeScreenInteractionType::Click;
		AnalyticsParam.LoadAtStartup = AutoLoadProjectComboBoxSelection;
		AnalyticsParam.ActiveSection = MainHomeSelection;
		AnalyticsParam.bHasProjectsOnMachine = bHasProjectsOnMachine;
		FHomeScreenInteractionAnalyticsUtils::RegisterInteractionUserAnalytics(AnalyticsParam);
	}

	const FHomeScreenSectionLink& CurrentRequestedLink = SectionLinks[InMainHomeSelection];
	const FString BaseURL = CurrentRequestedLink.BeforeLocaleURL;

	// Use the Editor Language for the WebPage language.
	const FString FullLocale = FInternationalization::Get().GetCurrentLanguage()->GetName();
	FString EndURL = CurrentRequestedLink.AfterLocaleURL;

	if (!bHasProjectsOnMachine && !CurrentRequestedLink.AfterLocaleNoProjectURL.IsEmpty())
	{
		EndURL = CurrentRequestedLink.AfterLocaleNoProjectURL;
	}

	const FString FinalURL = BaseURL / FullLocale / EndURL;

	if (WebBrowser.IsValid())
	{
		WebBrowser->LoadURL(FinalURL);
	}
}

void SHomeScreen::OnLoadAtStartupSettingChanged(EAutoLoadProject InAutoLoadOption)
{
	AutoLoadProjectComboBoxSelection = InAutoLoadOption;

	FHomeScreenInteractionAnalyticsParam AnalyticsParam;
	AnalyticsParam.HomeScreenElement = EHomeScreenElement::LoadAtStartup;
	AnalyticsParam.InteractionType = EHomeScreenInteractionType::Click;
	AnalyticsParam.LoadAtStartup = AutoLoadProjectComboBoxSelection;
	AnalyticsParam.ActiveSection = MainHomeSelection;
	AnalyticsParam.bHasProjectsOnMachine = bHasProjectsOnMachine;
	FHomeScreenInteractionAnalyticsUtils::RegisterInteractionUserAnalytics(AnalyticsParam);
}

FReply SHomeScreen::OnAutoLoadOptionChanged(EAutoLoadProject InAutoLoadOption)
{
	if (AutoLoadProjectComboBoxSelection == InAutoLoadOption)
	{
		return FReply::Handled();
	}

	// SetOnLoad should change the setting value and the setting class should call the OnChanged that will call out OnLoadAtStartupSettingChanged
	// This is done so that the setting class will decide to change the setting or not since there may be more logic there
	// If the Set is later on skipped for example we will still have the setting here and in the setting class synced
	if (OnGetStartupSettingDelegate.IsBound())
	{
		const FHomeScreenLoadAtStartupSetting& LoadAtStartupSetting = OnGetStartupSettingDelegate.Execute();
		LoadAtStartupSetting.SetOnLoadAtStartup(InAutoLoadOption == EAutoLoadProject::LastProject);
	}

	if (TSharedPtr<SWidget> ComboButtonMenu = ComboButtonMenuWeak.Pin())
	{
		FSlateApplication::Get().DismissMenuByWidget(ComboButtonMenu.ToSharedRef());
	}

	return FReply::Handled();
}

EVisibility SHomeScreen::IsAutoLoadOptionCheckVisible(EAutoLoadProject InAutoLoadOption) const
{
	return AutoLoadProjectComboBoxSelection == InAutoLoadOption ? EVisibility::Visible : EVisibility::Hidden;
}

bool SHomeScreen::IsCheckBoxCheckedOrHovered(const TSharedPtr<SCheckBox> InCheckBox) const
{
	return InCheckBox.IsValid() ? InCheckBox->IsHovered() || InCheckBox->IsChecked() : false;
}

FSlateColor SHomeScreen::GetMainSectionCheckBoxColor(const TSharedPtr<SCheckBox> InCheckBox) const
{
	return IsCheckBoxCheckedOrHovered(InCheckBox) ? FLinearColor::White : FLinearColor(1.f, 1.f, 1.f, 0.65f);
}

FSlateColor SHomeScreen::GetResourceAndSocialMediaButtonColor(const TSharedPtr<SButton> InButton) const
{
	if (InButton.IsValid() && InButton->IsHovered())
	{
		return FLinearColor::White;
	}

	return FLinearColor(1.f, 1.f, 1.f, 0.65f);
}

void SHomeScreen::CreateMainSectionCheckBox(TSharedPtr<SCheckBox>& OutCheckBox, EMainSectionMenu InMainHomeSelection, const FText& InText, const FSlateBrush* InImage)
{
	OutCheckBox = SNew(SCheckBox)
		.Visibility_Lambda([this, InMainHomeSelection] () { return SectionLinks.Contains(InMainHomeSelection) ? EVisibility::Visible : EVisibility::Collapsed; })
		.IsEnabled(this, &SHomeScreen::IsMainSectionEnabled, InMainHomeSelection)
		.Style(FAppStyle::Get(), "HomeScreen.MainMenuSectionButton")
		.IsChecked(this, &SHomeScreen::IsMainHomeSectionChecked, InMainHomeSelection)
		.OnCheckStateChanged(this, &SHomeScreen::OnMainHomeSectionChanged, InMainHomeSelection, false)
		.Padding(FMargin(8.f, 2.f));

	OutCheckBox->SetContent(
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 8.f, 0.f)
		[
			SNew(SImage)
			.Image(InImage)
			.ColorAndOpacity(this, &SHomeScreen::GetMainSectionCheckBoxColor, OutCheckBox)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
			.Text(InText)
			.ColorAndOpacity(this, &SHomeScreen::GetMainSectionCheckBoxColor, OutCheckBox)
		]);
}

void SHomeScreen::CreateResourceButtons(TSharedPtr<SButton>& OutButton, TAttribute<FString> InLink, const FText& InText, const FSlateBrush* InImage)
{
	SAssignNew(OutButton, SButton)
	.IsEnabled(this, &SHomeScreen::IsConnectedToInternet)
	.ButtonStyle(FAppStyle::Get(), "NoBorder")
	.ContentPadding(FMargin(0.f, 2.f))
	.OnClicked(this, &SHomeScreen::OnLinkClicked, InLink, EHomeScreenElement::ResourceButton);

	OutButton->SetContent(
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 8.f, 0.f)
		[
			SNew(SImage)
			.Image(InImage)
			.ColorAndOpacity(this, &SHomeScreen::GetResourceAndSocialMediaButtonColor, OutButton)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
			.Text(InText)
			.ColorAndOpacity(this, &SHomeScreen::GetResourceAndSocialMediaButtonColor, OutButton)
		]);
}

void SHomeScreen::CreateSocialMediaButtons(TSharedPtr<SButton>& OutButton, FString InLink, const FSlateBrush* InImage)
{
	SAssignNew(OutButton, SButton)
	.ButtonStyle(FAppStyle::Get(), "NoBorder")
	.OnClicked(this, &SHomeScreen::OnLinkClicked, InLink, EHomeScreenElement::SocialLinkButton);

	OutButton->SetContent(
			SNew(SImage)
			.Image(InImage)
			.ColorAndOpacity(this, &SHomeScreen::GetResourceAndSocialMediaButtonColor, OutButton));
}

void SHomeScreen::CreateWebBrowserWidgetIfNeeded()
{
	// WebBrowser already created or failed to create, early exit
	// Note that we check WebObject instead of WebBrowser to avoid showing error messages repeatedly in case the WebBrowserModule initialization or WebBrowser creation fails
	if (WebObject.IsValid())
	{
		return;
	}

	WebObject = TStrongObjectPtr(NewObject<UHomeScreenWeb>());
	WebObject->OnNavigationChanged().AddSP(this, &SHomeScreen::OnNavigateToSection);
	WebObject->OnTutorialProjectRequested().AddSP(this, &SHomeScreen::OnOpenGettingStartedProject);
	WebObject->OnOpenLink().AddSP(this, &SHomeScreen::OnWebLinkClicked);

	IWebBrowserModule& WebBrowserModule = IWebBrowserModule::Get();
	if (!IWebBrowserModule::IsAvailable() || !WebBrowserModule.IsWebModuleAvailable())
	{
		WebBrowserContainer->SetContent(
			SNew(STextBlock)
			.Text(LOCTEXT("WebBrowserModuleNotAvailable", "Failed to load the web browser module, can't display web content!"))
			.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
			.Justification(ETextJustify::Center)
		);
		return;
	}

	IWebBrowserSingleton* WebBrowserSingleton = WebBrowserModule.GetSingleton();

	FCreateBrowserWindowSettings WindowSettings;
	WindowSettings.bUseTransparency = true;
	TSharedPtr<IWebBrowserWindow> WebBrowserWindow = WebBrowserSingleton->CreateBrowserWindow(WindowSettings);
	if (!WebBrowserWindow.IsValid())
	{
		WebBrowserContainer->SetContent(
			SNew(STextBlock)
			.Text(LOCTEXT("WebBrowserWindowCreationIssue", "Failed to create the web browser window, can't display web content!"))
			.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
			.Justification(ETextJustify::Center)
		);
		return;
	}

#if !UE_BUILD_SHIPPING
	WebBrowserSingleton->SetDevToolsShortcutEnabled(true);
	WebBrowserWindow->OnCreateWindow().BindSPLambda(this, [](const TWeakPtr<IWebBrowserWindow>& InNewBrowserWindowWeak, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures)
		{
			if (const TSharedPtr<IWebBrowserWindow> NewBrowserWindow = InNewBrowserWindowWeak.Pin())
			{
				// Initialize a dialog
				auto DialogMainWindow = SNew(SWindow)
					.Title(FText::FromString(TEXT("Chrome Debugging Tools")))
					.ClientSize(FVector2D(700, 700))
					.SupportsMaximize(true)
					.SupportsMinimize(true)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(SWebBrowser, NewBrowserWindow.ToSharedRef())
						]
					];
				FSlateApplication::Get().AddWindow(DialogMainWindow);
				return true;
			}
			return false;
		});
#endif

	SAssignNew(WebBrowser, SWebBrowser, WebBrowserWindow)
		.OnLoadStarted(FSimpleDelegate::CreateSPLambda(this, [this]() { WebBrowser->ShowThrobber(); }))
		.OnLoadError(FSimpleDelegate::CreateSPLambda(this, [this]() { WebBrowser->HideThrobber(); }))
		.OnLoadCompleted(FSimpleDelegate::CreateSPLambda(this, [this]() { WebBrowser->HideThrobber(); }))
		.ShowControls(false)
		.SupportsTransparency(true)
		.ShowAddressBar(false);

	WebBrowserContainer->SetContent(WebBrowser.ToSharedRef());

	constexpr bool bIsPermanent = true;
	WebBrowser->BindUObject(TEXT("uebridge"), WebObject.Get(), bIsPermanent);

	// Skip the analytics when selecting the first section during the construction of the widget
	constexpr bool bSkipAnalytics = true;
	OnMainHomeSectionChanged(ECheckBoxState::Checked, MainHomeSelection, bSkipAnalytics);
}

TSharedRef<SWidget> SHomeScreen::CreateComboButtonMenuContentWidget()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("HomeScreenComboButtonMenu"))
	{
		FToolMenuContext HomeScreenComboContext;

		UHomeScreenContext* HomeScreenContextObject = NewObject<UHomeScreenContext>();
		HomeScreenContextObject->HomeScreen = SharedThis(this);

		HomeScreenComboContext.AddObject(HomeScreenContextObject);
		TSharedRef<SWidget> ComboButtonMenuRef = ToolMenus->GenerateWidget("HomeScreenComboButtonMenu", HomeScreenComboContext);
		ComboButtonMenuWeak = ComboButtonMenuRef;
		return ComboButtonMenuRef;
	}
	return SNullWidget::NullWidget;
}

void SHomeScreen::CheckInternetConnection()
{
	// Fire another request only if the previous one finished
	if (HttpRetryRequest.IsValid())
	{
		return;
	}

	bIsRequestFinished = false;

	HttpRetryRequest = HttpRetryManager->CreateRequest(
		FHttpRetrySystem::FRetryLimitCountSetting(3),
		FHttpRetrySystem::FRetryTimeoutRelativeSecondsSetting(3),
		FHttpRetrySystem::FRetryResponseCodes(),
		FHttpRetrySystem::FRetryVerbs(),
		FHttpRetrySystem::FRetryDomainsPtr());

	HttpRetryRequest->SetURL(TEXT("https://www.google.com/generate_204"));
	HttpRetryRequest->SetVerb("GET");

	HttpRetryRequest->OnProcessRequestComplete().BindLambda([this](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful)
		{
			bIsRequestFinished = true;
			bIsConnected = bWasSuccessful;
			HttpRetryRequest.Reset();
		});

	HttpRetryRequest->ProcessRequest();
}

bool SHomeScreen::IsConnectedToInternet() const
{
	return bIsConnected;
}

bool SHomeScreen::IsMainSectionEnabled(EMainSectionMenu InHomeSection) const
{
	return IsConnectedToInternet() || InHomeSection == EMainSectionMenu::Home;
}

int32 SHomeScreen::GetNoInternetIconIndex() const
{
	return bIsRequestFinished? 0 /** No Internet */ : 1 /** Retrying */;
}

void SHomeScreen::OnNavigateToSection(EMainSectionMenu InSectionToNavigate)
{
	OnMainHomeSectionChanged(ECheckBoxState::Checked, InSectionToNavigate);
}

void SHomeScreen::OnOpenGettingStartedProject()
{
	FHomeScreenInteractionAnalyticsParam AnalyticsParam;
	AnalyticsParam.HomeScreenElement = EHomeScreenElement::CreateStartupProjectButton;
	AnalyticsParam.InteractionType = EHomeScreenInteractionType::Click;
	AnalyticsParam.LoadAtStartup = AutoLoadProjectComboBoxSelection;
	AnalyticsParam.ActiveSection = MainHomeSelection;
	AnalyticsParam.bHasProjectsOnMachine = bHasProjectsOnMachine;
	FHomeScreenInteractionAnalyticsUtils::RegisterInteractionUserAnalytics(AnalyticsParam);

	FString TemplateRootFolders = FPaths::RootDir() + TEXT("Templates");
	FString UEIntroRootFolder = TemplateRootFolders / TEXT("TP_UEIntro_BP");
	const FString SearchString = UEIntroRootFolder / TEXT("*.") + FProjectDescriptor::GetExtension();
	TArray<FString> FoundProjectFiles;
	IFileManager::Get().FindFiles(FoundProjectFiles, *SearchString, /*Files=*/true, /*Directories=*/false);

	// There should be only 1 match
	if (FoundProjectFiles.Num() != 1)
	{
		return;
	}

	const FString TemplateProjectPath = UEIntroRootFolder / FoundProjectFiles[0];
	const FString DesiredProjectName = TEXT("UEIntroProject");

	// Get the default project creation folder
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (!DesktopPlatform)
	{
		return;
	}

	FString DefaultProjectFolder = DesktopPlatform->GetDefaultProjectCreationPath();

	// Find a unique name
	FString ProjectName = DesiredProjectName;
	FString NewProjectFolder = FPaths::Combine(DefaultProjectFolder, ProjectName);
	int32 Counter = 1;
	constexpr int32 MaxTries = 1000;

	while (FPaths::DirectoryExists(NewProjectFolder))
	{
		ProjectName = FString::Printf(TEXT("%s_%d"), *DesiredProjectName, Counter++);
		NewProjectFolder = FPaths::Combine(DefaultProjectFolder, ProjectName);
		if (Counter > MaxTries)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("TooManyTriesToFindAUniqueName", "Something went wrong when trying to find a unique name for the tutorial project.\nCheck your folder containing the projects and clean that up of unneeded tutorial projects."));
			return;
		}
	}

	const FString ProjectPath = DefaultProjectFolder;
	const FString Filename = ProjectName + TEXT(".") + FProjectDescriptor::GetExtension();
	FString ProjectFilename = FPaths::Combine(*ProjectPath, *ProjectName, *Filename);
	FPaths::MakePlatformFilename(ProjectFilename);

	FText FailReason, FailLog;
	FProjectInformation ProjectInfo;
	ProjectInfo.TemplateCategory = TEXT("Game");
	ProjectInfo.TemplateFile = TemplateProjectPath;
	ProjectInfo.ProjectFilename = ProjectFilename;

	if (!FGameProjectGenerationModule::Get().CreateProject(ProjectInfo, FailReason, FailLog))
	{
		SOutputLogDialog::Open(LOCTEXT("CreateProject", "Create Project"), FailReason, FailLog, FText::GetEmpty());
		return;
	}

	// Successfully created the project. Update the last created location string.
	FString CreatedProjectPath = FPaths::GetPath(FPaths::GetPath(ProjectFilename));

	// If the original path was the drives root (ie: C:/) the double path call strips the last /
	if (CreatedProjectPath.EndsWith(":"))
	{
		CreatedProjectPath.AppendChar('/');
	}

	UEditorSettings* Settings = GetMutableDefault<UEditorSettings>();
	Settings->CreatedProjectPaths.Remove(CreatedProjectPath);
	Settings->CreatedProjectPaths.Insert(CreatedProjectPath, 0);
	Settings->PostEditChange();

	// Open the project
	FText FailReasonOpen;
	if (FGameProjectGenerationModule::Get().OpenProject(ProjectFilename, FailReasonOpen))
	{
		// Successfully opened the project, the editor is closing.
		// Close this window in case something prevents the editor from closing (save dialog, quit confirmation, etc)
		if (FApp::HasProjectName())
		{
			TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	
			if (ContainingWindow.IsValid())
			{
				ContainingWindow->RequestDestroyWindow();
			}
		}
	}
	else
	{
		FString ErrorString = FailReasonOpen.ToString();
		UE_LOGF(LogTemp, Log, "%ls", *ErrorString);
		FMessageDialog::Open(EAppMsgType::Ok, FailReasonOpen);
	}
}

FText SHomeScreen::GetAutoLoadProjectComboBoxLabelText() const
{
	return AutoLoadProjectComboBoxSelection == EAutoLoadProject::HomeScreen ? LOCTEXT("HomeScreenComboHomePanel", "Home Panel") : LOCTEXT("HomeScreenComboMostRecentProject", "Most Recent Project");
}

FReply SHomeScreen::OnInternetConnectionRetried()
{
	CheckInternetConnection();
	return FReply::Handled();
}

bool SHomeScreen::HasAlreadyLatestEngineProject() const
{
	IDesktopPlatform* DesktopPlatformModule = FDesktopPlatformModule::Get();
	if (!DesktopPlatformModule)
	{
		return false;
	}

	// Get all engine installation and projects known by them
	TMap<FString, FString> EngineInstallations;
	DesktopPlatformModule->EnumerateEngineInstallations(EngineInstallations);
	
	for (TMap<FString, FString>::TConstIterator Iter(EngineInstallations); Iter; ++Iter)
	{
		TArray<FString> ProjectFiles;

		if (DesktopPlatformModule->EnumerateProjectsKnownByEngine(Iter.Key(), false, ProjectFiles))
		{
			FString RootDir;
			if (DesktopPlatformModule->GetEngineRootDirFromIdentifier(Iter.Key(), RootDir))
			{
				// Iterate over each found project and return true if one of them is made with this engine version
				for (const FString& Project : ProjectFiles)
				{
					const FString ProjectFilename = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Project);
					if (!FPaths::FileExists(ProjectFilename) || ProjectFilename.Contains(RootDir))
					{
						continue;
					}
                    
                    FString Identifier;
                    DesktopPlatformModule->GetEngineIdentifierForProject(ProjectFilename, Identifier);
                    
                    FEngineVersion ProjectEngineVersion;
                    
                    if (DesktopPlatformModule->IsStockEngineRelease(Identifier))
                    {
                    	DesktopPlatformModule->TryParseStockEngineVersion(Identifier, ProjectEngineVersion);
                    }
                    
                    if (ProjectEngineVersion.IsEmpty())
                    {
                    	FString RootProjectDir;
                    	DesktopPlatformModule->GetEngineRootDirFromIdentifier(Identifier, RootProjectDir);
                    	if (!DesktopPlatformModule->TryGetEngineVersion(RootProjectDir, ProjectEngineVersion))
                    	{
                    		continue;
                    	}
                    }
                    
                    const FString EngineVersionString = FEngineVersion::Current().ToString(EVersionComponent::Minor);
                    const FString ProjectEngineVersionString = ProjectEngineVersion.ToString(EVersionComponent::Minor);
                    
                    if (EngineVersionString == ProjectEngineVersionString)
                    {
                    	return true;
                    }
				}
			}
		}
	}

	return false;
}

void SHomeScreen::StartInternetConnectionTickCheck()
{
	if (GEditor && GEditor->IsTimerManagerValid() && !CheckInternetConnectionTimerHandle.IsValid())
	{
		constexpr float TimerManagerCallRate = 5.f;
		constexpr bool bLoop = true;

		GEditor->GetTimerManager()->SetTimer(
			CheckInternetConnectionTimerHandle,
			FTimerDelegate::CreateSPLambda(this, [this]()
				{
					if (bIsRequestFinished)
					{
						if (bIsConnected || bForceRetry)
						{
							bForceRetry = false;
							TimerManagerCountOnceDisconnected = 0;
							CheckInternetConnection();
						}
						else
						{
							// When disconnected retry connection less times than when you are connected
							TimerManagerCountOnceDisconnected += 1;
							bForceRetry = TimerManagerCountOnceDisconnected == MaxTimerManagerCountOnceDisconnectedBeforeRetry;
						}
					}
				}),
				TimerManagerCallRate,
				bLoop);
	}
}

void SHomeScreen::StopInternetConnectionTickCheck()
{
	if (GEditor && GEditor->IsTimerManagerValid() && CheckInternetConnectionTimerHandle.IsValid())
	{
		GEditor->GetTimerManager()->ClearTimer(CheckInternetConnectionTimerHandle);
		CheckInternetConnectionTimerHandle.Invalidate();
	}
}

void SHomeScreen::RegisterOnDestructedAnalyticEvent()
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		bool bIsRequestingMainFrameControl = false;

		const TArray<IEditorMainFrameProvider*> MainFrameProviders = IModularFeatures::Get().GetModularFeatureImplementations<IEditorMainFrameProvider>(IEditorMainFrameProvider::GetModularFeatureName());		
		for (const IEditorMainFrameProvider* Provider : MainFrameProviders)
		{
			// If a provider is requesting main frame control assume that there is no project opened
			// Since these provider are used as project creation/opening when opening up the engine with no project
			// We can't really use FApp::HasProjectName since there are cases where the project has a name, but it's not a user project
			if (Provider && Provider->IsRequestingMainFrameControl())
			{
				bIsRequestingMainFrameControl = true;
				break;
			}
		}

		// If a provider is requesting control consider the current session as a no project one
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ClosedWithoutProject"), bIsRequestingMainFrameControl));

		// Register the last section the user was on
		static const UEnum* MainSectionEnum = StaticEnum<EMainSectionMenu>();
		Attributes.Add(FAnalyticsEventAttribute(TEXT("LastSection"),  MainSectionEnum->GetNameByValue((int64)MainHomeSelection)));

		// Record the destroyed event of the HomeScreen
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.HomeScreen.Destroyed"), Attributes);
	}
}

FDelayedAutoRegisterHelper SHomeScreen::LoadStartupComboButtonRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit, 
	[] {
			UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
				{
					FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
					static const FName LoadStartupComboButtonName("HomeScreenComboButtonMenu");
					UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(LoadStartupComboButtonName);
					FToolMenuSection& Section = Menu->AddSection("StartupSection");

					Section.AddDynamicEntry(
						TEXT("HomeTabDynamic"),
						FNewToolMenuSectionDelegate::CreateLambda([] (FToolMenuSection& InSection)
							{
								if (UHomeScreenContext* Context = InSection.FindContext<UHomeScreenContext>())
								{
									if (const TSharedPtr<SHomeScreen> HomeScreen = Context->HomeScreen.Pin())
									{
										const TSharedRef<SHomeScreen> HomeScreenRef = HomeScreen.ToSharedRef();

										TSharedRef<SButton> HomeTabWidget = SNew(SButton)
											.ButtonStyle(FAppStyle::Get(), "HomeScreen.ComboButton.MenuButton")
											.ContentPadding(FMargin(12.f, 10.f))
											.OnClicked(HomeScreenRef, &SHomeScreen::OnAutoLoadOptionChanged, EAutoLoadProject::HomeScreen)
											[
												SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
												.Padding(0.f, 0.f, 8.f, 0.f)
												.AutoWidth()
												[
													SNew(SImage)
													.Image(FAppStyle::GetBrush("Icons.Check"))
													.Visibility(HomeScreenRef, &SHomeScreen::IsAutoLoadOptionCheckVisible, EAutoLoadProject::HomeScreen)
												]

												+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(STextBlock)
													.Text(LOCTEXT("HomeScreenHomePanel", "Home Panel"))
													.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
													.ColorAndOpacity(FStyleColors::White)
												]
											];

										InSection.AddEntry(FToolMenuEntry::InitWidget(
											TEXT("HomeTab"),
											SNew(SBox).Padding(8.f, 0.f, 8.f, 4.f)[HomeTabWidget],
											FText::GetEmpty()));
									}
								}
							})
						);


					Section.AddDynamicEntry(
						TEXT("MostRecentProjectDynamic"),
						FNewToolMenuSectionDelegate::CreateLambda([] (FToolMenuSection& InSection)
							{
								if (UHomeScreenContext* Context = InSection.FindContext<UHomeScreenContext>())
								{
									if (const TSharedPtr<SHomeScreen> HomeScreen = Context->HomeScreen.Pin())
									{
										const TSharedRef<SHomeScreen> HomeScreenRef = HomeScreen.ToSharedRef();

										TSharedRef<SButton> MostRecentProjectWidget = SNew(SButton)
											.ButtonStyle(FAppStyle::Get(), "HomeScreen.ComboButton.MenuButton")
											.ContentPadding(FMargin(12.f, 10.f))
											.OnClicked(HomeScreenRef, &SHomeScreen::OnAutoLoadOptionChanged, EAutoLoadProject::LastProject)
											[
												SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
												.Padding(0.f, 0.f, 8.f, 0.f)
												.AutoWidth()
												[
													SNew(SImage)
													.Image(FAppStyle::GetBrush("Icons.Check"))
													.Visibility(HomeScreenRef, &SHomeScreen::IsAutoLoadOptionCheckVisible, EAutoLoadProject::LastProject)
												]

												+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(STextBlock)
													.Text(LOCTEXT("HomeScreenMostRecentProject", "Most Recent Project"))
													.Font(FAppStyle::GetFontStyle("HomeScreen.Font.DefaultSize"))
													.ColorAndOpacity(FStyleColors::White)
												]
											];

										InSection.AddEntry(FToolMenuEntry::InitWidget(
											TEXT("MostRecentProject"),
											SNew(SBox).Padding(8.f, 0.f)[MostRecentProjectWidget],
											FText::GetEmpty()));
									}
								}
							})
						);
				}));
		}
);

#undef LOCTEXT_NAMESPACE
