// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/TimerHandle.h"
#include "Settings/HomeScreenCommon.h"
#include "HomeScreenAnalyticsUtils.h"
#include "HomeScreenWeb.h"
#include "HttpRetrySystem.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

class SBox;
class SButton;
class SCheckBox;
class SComboButton;
class SDockTab;
class SVerticalBox;
class SWebBrowser;
class UHomeScreenWeb;
template <typename ItemType> class STileView;

class SHomeScreen : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SHomeScreen)
		: _ResourceCommunityEntries()
		, _SectionLinks()
		, _AlwaysShowRecentList(false)
		, _RecentProjectList(SNullWidget::NullWidget)
		, _UnrealLogoBrush(nullptr)
		, _OnGetStartupSetting()
	{}
		SLATE_ARGUMENT(TArray<FHomeScreenResourceCommunityEntry>, ResourceCommunityEntries)
		SLATE_ARGUMENT(FHomeScreenSectionToLinkMap, SectionLinks)
		SLATE_ARGUMENT(bool, AlwaysShowRecentList)
		SLATE_ARGUMENT(TSharedRef<SWidget>, RecentProjectList)
		SLATE_ARGUMENT(const FSlateBrush*, UnrealLogoBrush)
		SLATE_ARGUMENT(FOnGetStartupSetting, OnGetStartupSetting)
	SLATE_END_ARGS()

	virtual ~SHomeScreen() override;

	/** Constructs the HomeScreen widget */
	void Construct(const FArguments& InArgs);

	/** Used when the HomeScreen tab becomes visible to start the internet connection check or to stop it when it goes in the background */
	void OnTabForegrounded(TSharedPtr<SDockTab> NewForegroundTab, TSharedPtr<SDockTab> BackgroundTab);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	/** Opens the Create Project dialog */
	FReply OnCreateProjectDialog(bool bInAllowProjectOpening, bool bInAllowProjectCreation);

	/** Handle the clicked links event */
	FReply OnLinkClicked(TAttribute<FString> InURL, const EHomeScreenElement InElement) const;
	
	/** Handle the clicked links event */
	FReply OnLinkClicked(FString InURL, const EHomeScreenElement InElement) const;

	/** Handle a web link being clicked */
	void OnWebLinkClicked(FString InURL) const;

	/** Checks if a main home section is selected */
	ECheckBoxState IsMainHomeSectionChecked(EMainSectionMenu InMainHomeSelection) const;

	/** Handles changes to the main home section */
	void OnMainHomeSectionChanged(ECheckBoxState InCheckBoxState, EMainSectionMenu InMainHomeSelection, bool bSkipAnalytics = false);

	/** Callback from the HomeScreenSettings when the LoadAtStartup property change */
	void OnLoadAtStartupSettingChanged(EAutoLoadProject InAutoLoadOption);

	/** Handles changes to autoload last project selection */
	FReply OnAutoLoadOptionChanged(EAutoLoadProject InAutoLoadOption);

	/** True if the current AutoLoad option match the given InAutoLoadOption */
	EVisibility IsAutoLoadOptionCheckVisible(EAutoLoadProject InAutoLoadOption) const;

	/** Checks if a checkbox is checked or hovered */
	bool IsCheckBoxCheckedOrHovered(const TSharedPtr<SCheckBox> InCheckBox) const;

	/** Gets the color of the main section checkbox icon and text */
	FSlateColor GetMainSectionCheckBoxColor(const TSharedPtr<SCheckBox> InCheckBox) const;

	/** Gets the color of resource and social media section icon and text */
	FSlateColor GetResourceAndSocialMediaButtonColor(const TSharedPtr<SButton> InButton) const;

	/** Creates a main section checkboxes */
	void CreateMainSectionCheckBox(TSharedPtr<SCheckBox>& OutCheckBox, EMainSectionMenu InMainHomeSelection, const FText& InText, const FSlateBrush* InImage);

	/** Creates resource buttons */
	void CreateResourceButtons(TSharedPtr<SButton>& OutButton, TAttribute<FString> InLink, const FText& InText, const FSlateBrush* InImage);

	/** Creates social media buttons */
	void CreateSocialMediaButtons(TSharedPtr<SButton>& OutButton, FString InLink, const FSlateBrush* InImage);

	/** Creates the webbrowser widget if not initialized yet */
	void CreateWebBrowserWidgetIfNeeded();

	/** Creates the content widget for the combo button menu */
	TSharedRef<SWidget> CreateComboButtonMenuContentWidget();

	/** Checks the internet connection */
	void CheckInternetConnection();

	/** Returns whether the system is connected to the internet */
	bool IsConnectedToInternet() const;

	/** Returns whether the main section is enabled */
	bool IsMainSectionEnabled(EMainSectionMenu InHomeSection) const;

	/** Gets the index of the SWidgetSwitcher for the no internet icon, either NoInternet or Loading when retrying */
	int32 GetNoInternetIconIndex() const;

	/** Handles navigation to a section from a web request */
	void OnNavigateToSection(EMainSectionMenu InSectionToNavigate);

	/** Handles the getting started template project creation */
	void OnOpenGettingStartedProject();

	/** Gets the label text for the autoload project combo box */
	FText GetAutoLoadProjectComboBoxLabelText() const;

	/** Executed when clicking on the Reconnect button when no internet connection is detected */
	FReply OnInternetConnectionRetried();

	/** Whether the current user already created project with this Engine version */
	bool HasAlreadyLatestEngineProject() const;

	/** Create and bind a function to check for internet connection from the TimerManager */
	void StartInternetConnectionTickCheck();

	/** Unbind the function to check for internet connection from the TimerManager */
	void StopInternetConnectionTickCheck();

	/** Register an analytic event when the HomeScreen is destroyed */
	void RegisterOnDestructedAnalyticEvent();

private:
	/** Registers the LoadStartup ComboButton Menu */
	static FDelayedAutoRegisterHelper LoadStartupComboButtonRegistration;

private:
	/** Currently selected main home section */
	EMainSectionMenu MainHomeSelection = EMainSectionMenu::Home;

	/** Main section checkboxes */
	TSharedPtr<SCheckBox> HomeCheckBox;
	TSharedPtr<SCheckBox> NewsCheckBox;
	TSharedPtr<SCheckBox> GettingStartedCheckBox;
	TSharedPtr<SCheckBox> SampleProjectsCheckBox;

	/** Handle holding the OnTabForegrounded callback */
	FDelegateHandle OnTabForegroundedHandle;

	/** Http retry system used to check Internet connection */
	TSharedPtr<FHttpRetrySystem::FManager> HttpRetryManager;
	TSharedPtr<FHttpRetrySystem::FRequest> HttpRetryRequest;

	/** Combo box for autoload project selection */
	TSharedPtr<SComboButton> AutoLoadProjectComboBox;

	/** Current combo button menu */
	TWeakPtr<SWidget> ComboButtonMenuWeak;

	/** VerticalBox holding the ResourceAndCommunity buttons */
	TSharedPtr<SVerticalBox> ResourceAndCommunityVerticalBox;

	/** Timer handle for checking internet connection */
	FTimerHandle CheckInternetConnectionTimerHandle;

	/** Path of the currently selected project */
	FString CurrentSelectedProjectPath;

	/** Whether the system is connected to the internet */
	bool bIsConnected = true;

	/** Whether the last request has finished */
	bool bIsRequestFinished = true;

	/** Whether the current user has projects on their machine */
	bool bHasProjectsOnMachine = false;

	/** How many times we executed the function without being connected */
	int32 TimerManagerCountOnceDisconnected = 0;

	/** Max TimerManager function execution before retrying once disconnected */
	int32 MaxTimerManagerCountOnceDisconnectedBeforeRetry = 5;

	/** Whether to force a retry on the connection */
	bool bForceRetry = false;

	/** Selection for autoload project combo box */
	EAutoLoadProject AutoLoadProjectComboBoxSelection;

	/** Web browser lazy-loading container */
	TSharedPtr<SBox> WebBrowserContainer;

	/** Web browser widget instance */
	TSharedPtr<SWebBrowser> WebBrowser;

	/** Bridge object for browser navigation */
	TStrongObjectPtr<UHomeScreenWeb> WebObject;

	/** HomeScreen start up setting getter */
	FOnGetStartupSetting OnGetStartupSettingDelegate;

	/** Section x link combination */
	FHomeScreenSectionToLinkMap SectionLinks;
};
