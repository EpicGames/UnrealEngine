// Copyright Epic Games, Inc. All Rights Reserved.

#include "MainFrameModule.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "GameProjectGenerationModule.h"
#include "MessageLogModule.h"
#include "MRUFavoritesList.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Sound/SoundBase.h"
#include "ISourceCodeAccessor.h"
#include "ISourceCodeAccessModule.h"
#include "Menus/MainMenu.h"
#include "Frame/RootWindowLocation.h"
#include "Kismet2/CompilerResultsLog.h"
#include "IHotReload.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Framework/Commands/GenericCommands.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Editor/EditorPerformanceSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "ToolMenuContext.h"
#include "Toolkits/FConsoleCommandExecutor.h"
#include "Misc/MessageDialog.h"
#include "ProfilingDebugging/StallDetector.h"
#include "Interfaces/IEditorMainFrameProvider.h"

// HomeScreen Includes
#include "HomeScreen/HomeScreenWeb.h"
#include "HomeScreen/SHomeScreen.h" 
#include "Internationalization/Culture.h"
#include "Menus/SettingsMenu.h"
#include "Settings/EditorStyleSettings.h"
#include "Settings/HomeScreenSettings.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"

DEFINE_LOG_CATEGORY(LogMainFrame);
#define LOCTEXT_NAMESPACE "FMainFrameModule"

namespace UE::Editor::HomeScreen
{
	static FAutoConsoleVariable CVarEnableHomeScreen(
	TEXT("HomeScreen.EnableHomeScreen"),
	true,
	TEXT("Whether or not to enable the new Home Panel. Value can be changed in the config file"),
	ECVF_ReadOnly
);
}

static FAutoConsoleCommand ResizeMainFrameCommand(
	TEXT("Editor.ResizeMainFrame"),
	TEXT(""),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FMainFrameModule::HandleResizeMainFrameCommand)
);

const FText StaticGetApplicationTitle( const bool bIncludeGameName )
{
	static const FText ApplicationTitle = NSLOCTEXT("UnrealEditor", "ApplicationTitle", "Unreal Editor");

	if (bIncludeGameName && FApp::HasProjectName())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("GameName"), FText::FromString( FString( FApp::GetProjectName())));
		Args.Add(TEXT("AppTitle"), ApplicationTitle);

		const EBuildConfiguration BuildConfig = FApp::GetBuildConfiguration();

		if (BuildConfig != EBuildConfiguration::Shipping && BuildConfig != EBuildConfiguration::Development && BuildConfig != EBuildConfiguration::Unknown)
		{
			Args.Add( TEXT("Config"), EBuildConfigurations::ToText(BuildConfig));

			return FText::Format( NSLOCTEXT("UnrealEditor", "AppTitleGameNameWithConfig", "{GameName} [{Config}] - {AppTitle}"), Args );
		}

		return FText::Format( NSLOCTEXT("UnrealEditor", "AppTitleGameName", "{GameName} - {AppTitle}"), Args );
	}

	return ApplicationTitle;
}

namespace HomeScreenHelper
{
	// Get the default UE home screen resource entries data
	TArray<FHomeScreenResourceCommunityEntry> GetUEResourceEntries()
	{
		TArray<FHomeScreenResourceCommunityEntry> DefaultEntries;

		FString EngineVersionString = FEngineVersion::Current().ToString(EVersionComponent::Minor);
		EngineVersionString = EngineVersionString.Replace(TEXT("."), TEXT("-"));

		// Forum
		FHomeScreenResourceCommunityEntry ForumEntry;
		ForumEntry.DisplayName = LOCTEXT("ProjectHomeForumsButton", "Forums");
		ForumEntry.Link = TAttribute<FString>::CreateLambda([]()
			{
				const FString FullLocale = FInternationalization::Get().GetCurrentLanguage()->GetName();
				const FString ForumLink = 
					TEXT("https://forums.unrealengine.com/categories?tag=unreal-engine&locale=")
					+ FullLocale;
				return ForumLink;
			});
		ForumEntry.IconBrushName = TEXT("HomeScreen.Forum");
		DefaultEntries.Add(ForumEntry);

		// Documentation
		FHomeScreenResourceCommunityEntry DocumentationEntry;
		DocumentationEntry.DisplayName = LOCTEXT("ProjectHomeDocumentation", "Documentation");
		DocumentationEntry.Link = TAttribute<FString>::CreateLambda([EngineVersionString]()
			{
				const FString FullLocale = FInternationalization::Get().GetCurrentLanguage()->GetName();
				const FString DocumentationLink = 
					TEXT("https://dev.epicgames.com/documentation/")
					+ FullLocale
					+ TEXT("/unreal-engine/unreal-engine-")
					+  EngineVersionString
					+ TEXT("-documentation");
				return DocumentationLink;
			});
		DocumentationEntry.IconBrushName = TEXT("HomeScreen.Documentation");
		DefaultEntries.Add(DocumentationEntry);

		// Tutorial
		FHomeScreenResourceCommunityEntry TutorialEntry;
		TutorialEntry.DisplayName = LOCTEXT("ProjectHomeTutorialsButton", "Tutorials");
		TutorialEntry.Link = TAttribute<FString>::CreateLambda([]()
			{
				const FString FullLocale = FInternationalization::Get().GetCurrentLanguage()->GetName();
				const FString TutorialLink = 
					TEXT("https://dev.epicgames.com/community/unreal-engine/learning?locale=")
					+ FullLocale;
				return TutorialLink;
			});
		TutorialEntry.IconBrushName = TEXT("HomeScreen.Tutorial");
		DefaultEntries.Add(TutorialEntry);

		// Roadmap
		FHomeScreenResourceCommunityEntry RoadmapEntry;
		RoadmapEntry.DisplayName = LOCTEXT("ProjectHomeRoadmapButton", "Roadmap");
		// This link will always point to the latest roadmap and its intended to always point to latest
		RoadmapEntry.Link = TEXT("https://unrealengine.com/roadmap");
		RoadmapEntry.IconBrushName = TEXT("HomeScreen.Roadmap");
		DefaultEntries.Add(RoadmapEntry);

		// Release
		FHomeScreenResourceCommunityEntry ReleaseEntry; 
		ReleaseEntry.DisplayName = LOCTEXT("ProjectHomeReleaseNotesButton", "Release Notes");
		ReleaseEntry.Link = TAttribute<FString>::CreateLambda([EngineVersionString]()
			{
				const FString FullLocale = FInternationalization::Get().GetCurrentLanguage()->GetName();
				const FString ReleaseLink = 
					TEXT("https://dev.epicgames.com/documentation/")
					+ FullLocale
					+ TEXT("/unreal-engine/unreal-engine-")
					+  EngineVersionString
					+ TEXT("-release-notes");
				return ReleaseLink;
			});
		ReleaseEntry.IconBrushName = TEXT("HomeScreen.ReleaseNotes");
		DefaultEntries.Add(ReleaseEntry);
		
		return DefaultEntries;
	}

	// Get the default UE home screen main section entries data
	FHomeScreenSectionToLinkMap GetUESectionLinks()
	{
		FHomeScreenSectionToLinkMap OutSectionLinks;
		const FString EmptyURL = TEXT("");
		const FString BaseURL = TEXT("https://editor.unrealengine.com");
		OutSectionLinks.Add(EMainSectionMenu::News, 
			{ BaseURL,  TEXT("news"), EmptyURL});
		OutSectionLinks.Add(EMainSectionMenu::GettingStarted, 
			{ BaseURL,  TEXT("get-started-with-unreal-engine"), EmptyURL});
		OutSectionLinks.Add(EMainSectionMenu::SampleProjects, 
			{ BaseURL,  TEXT("samples"), EmptyURL});
		OutSectionLinks.Add(EMainSectionMenu::Home, 
			{ BaseURL,  TEXT("recent-projects"), TEXT("no-projects")});
		return OutSectionLinks;
	}

	TSharedRef<SWidget> GetRecentProjectListWidget()
	{
		return SNew(SBox)
			.Padding(8.f)
			[
				SNew(SBox)
				.HeightOverride(172.f)
				[
					// Embedded should not show the HeaderBar and should not allow scrolling
					FGameProjectGenerationModule::Get()
						.CreateEmbeddedProjectBrowser()
				]
			];
	}

	// Get the default UE home screen settings
	FHomeScreenWidgetSettings GetHomeScreenSettings()
	{
		FHomeScreenWidgetSettings HomeScreenWidgetSettings;
		HomeScreenWidgetSettings.ResourcesEntries = GetUEResourceEntries();
		HomeScreenWidgetSettings.SectionLinks = GetUESectionLinks();
		HomeScreenWidgetSettings.bAlwaysShowRecentList = false;
		HomeScreenWidgetSettings.RecentProjectListWidget = GetRecentProjectListWidget();
		HomeScreenWidgetSettings.UnrealLogoBrush = FAppStyle::GetBrush("HomeScreen.UnrealLogo");
		HomeScreenWidgetSettings.OnGetStartupSetting.BindUObject(GetMutableDefault<UHomeScreenSettings>(), &UHomeScreenSettings::GetLoadAtStartupSetting);
		return HomeScreenWidgetSettings;
	}
}

/* FEditorHomeScreenProvider
 *****************************************************************************/

void FEditorHomeScreenProvider::Register()
{
	IModularFeatures::Get().RegisterModularFeature(IHomeScreenProvider::GetModularFeatureName(), this);
}

void FEditorHomeScreenProvider::UnRegister()
{
	IModularFeatures::Get().UnregisterModularFeature(IHomeScreenProvider::GetModularFeatureName(), this);
}

bool FEditorHomeScreenProvider::IsRequestingHomeScreenPageControl() const
{
	return true;
}

int32 FEditorHomeScreenProvider::GetHomeScreenPriority()
{
	// Lower priority since this is the default behavior
	return -1;
}

FHomeScreenWidgetSettings FEditorHomeScreenProvider::GetHomeScreenWidgetSettings() const
{
	// No need to add settings as we want them to be the default
	return HomeScreenHelper::GetHomeScreenSettings();
}

/* FProjectDialogProvider
 *****************************************************************************/

void FProjectDialogProvider::Register()
{
	IModularFeatures::Get().RegisterModularFeature(IEditorMainFrameProvider::GetModularFeatureName(), this);
}

void FProjectDialogProvider::UnRegister()
{
	IModularFeatures::Get().UnregisterModularFeature(IEditorMainFrameProvider::GetModularFeatureName(), this);
}

bool FProjectDialogProvider::IsRequestingMainFrameControl() const
{
	return !FApp::HasProjectName();
}

FMainFrameWindowOverrides FProjectDialogProvider::GetDesiredWindowConfiguration() const
{
	bool bIsHomeScreenEnabled = UE::Editor::HomeScreen::IsHomeScreenEnabled();

	FMainFrameWindowOverrides ProjectDialogWindowConfig;

	// Force tabs restored from layout that have no window (the LevelEditor tab) to use a docking area with
	// embedded title area content.  We need to override the behavior here because we're creating the actual
	// window ourselves instead of letting the tab management system create it for us.
	ProjectDialogWindowConfig.bEmbedTitleAreaContent = false;

	// Do not maximize the window initially. Keep a small dialog feel.
	ProjectDialogWindowConfig.bInitiallyMaximized = bIsHomeScreenEnabled;

	ProjectDialogWindowConfig.WindowSize = bIsHomeScreenEnabled ? TOptional<FVector2D>() : FMainFrameModule::GetProjectBrowserWindowSize();

	ProjectDialogWindowConfig.bIsUserSizable = true;
	ProjectDialogWindowConfig.bSupportsMaximize = true;
	ProjectDialogWindowConfig.bSupportsMinimize = true;
	ProjectDialogWindowConfig.CenterRules = EAutoCenter::PreferredWorkArea;
	// When opening the project dialog, show "Project Browser" in the window title
	ProjectDialogWindowConfig.WindowTitle = LOCTEXT("ProjectBrowserDialogTitle", "Unreal Project Browser");
	if (bIsHomeScreenEnabled)
	{
		const FString EngineVersionString = FEngineVersion::Current().ToString(EVersionComponent::Minor);
		ProjectDialogWindowConfig.WindowTitle = FText::Format(LOCTEXT("HomeScreenDialogTitle", "Unreal Engine {0}"), FText::FromString(EngineVersionString));;
	}

	return ProjectDialogWindowConfig;
}

TSharedRef<SWidget> FProjectDialogProvider::CreateMainFrameContentWidget() const
{
	if (UE::Editor::HomeScreen::IsHomeScreenEnabled())
	{
		return IMainFrameModule::Get().CreateHomeScreenWidget({}, HomeScreenHelper::GetHomeScreenSettings());
	}

	return FGameProjectGenerationModule::Get().CreateGameProjectDialog(/*bAllowProjectOpening=*/true, /*bAllowProjectCreate=*/true);
}

/* IMainFrameModule implementation
 *****************************************************************************/

void FMainFrameModule::CreateDefaultMainFrame(const bool bStartImmersive, const bool bStartPIE)
{
	CreateDefaultMainFrameAuxiliary(bStartImmersive, bStartPIE, /*bIsBeingRecreated*/false);
}

void FMainFrameModule::RecreateDefaultMainFrame(const bool bStartImmersive, const bool bStartPIE)
{
	check(!bRecreatingDefaultMainFrame);
	TGuardValue<bool> GuardRecreatingDefaultMainFrame(bRecreatingDefaultMainFrame, true);

	// Clean previous default main frame
	if (IsWindowInitialized())
	{
		// Clean FGlobalTabmanager
		FGlobalTabmanager::Get()->CloseAllAreas();

		// Clean FSlateApplication
		FSlateApplication::Get().CloseAllWindowsImmediately();
	}
	// (Re-)create default main frame
	CreateDefaultMainFrameAuxiliary(bStartImmersive, bStartPIE, /*bIsBeingRecreated*/true);
}

void FMainFrameModule::ReconstructDefaultMainFrameContent(const bool bStartImmersive, const bool bStartPIE)
{
	// bRecreatingDefaultMainFrame make sure that the DestroyWindow called on the RootWindow will not be executed
	check(!bRecreatingDefaultMainFrame);
	TGuardValue<bool> GuardRecreatingDefaultMainFrame(bRecreatingDefaultMainFrame, true);

	// Clean previous default main frame
	if (IsWindowInitialized())
	{
		// Clean FGlobalTabmanager
		FGlobalTabmanager::Get()->CloseAllAreas();

		// Iterate every TopLevelWindows and destroy them unless the window is not regular or is the MainFrame window
		TArray<TSharedRef<SWindow>> TopLevelWindows = FSlateApplication::Get().GetTopLevelWindows();
		DestroyRegularAndNonRootWindows(TopLevelWindows);

		// Iterate every visible windows and destroy them unless the window is not regular or is the MainFrame window
		TArray<TSharedRef<SWindow>> OutVisibleWindows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(OutVisibleWindows);
		DestroyRegularAndNonRootWindows(OutVisibleWindows);

		// (Re-)populate the default main frame
		CreateDefaultMainFrameAuxiliary(bStartImmersive, bStartPIE, false, true);
	}
}

void FMainFrameModule::CreateDefaultMainFrameAuxiliary(const bool bStartImmersive, const bool bStartPIE, const bool bIsBeingRecreated, const bool bIsJustCreatingContent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMainFrameModule::CreateDefaultMainFrame);

	if (!IsWindowInitialized() || bIsJustCreatingContent)
	{
		FRootWindowLocation DefaultWindowLocation;

		FMainFrameWindowOverrides WindowConfig;
		bool bShowStartupDialogInPlaceOfMainEditor = false;
		TSharedPtr<SWidget> MainFrameContent;

		FGameProjectGenerationModule::Get();

		TArray<IEditorMainFrameProvider*> MainFrameProviders = IModularFeatures::Get().GetModularFeatureImplementations<IEditorMainFrameProvider>(IEditorMainFrameProvider::GetModularFeatureName());		
		for (IEditorMainFrameProvider* Provider: MainFrameProviders)
		{
			if (Provider && Provider->IsRequestingMainFrameControl())
			{
				WindowConfig = Provider->GetDesiredWindowConfiguration();
				MainFrameContent = Provider->CreateMainFrameContentWidget();
				bShowStartupDialogInPlaceOfMainEditor = true;

				// We can only have one main frame provider, which means we just use  the first active one
				break;
			}
		}

		if (WindowConfig.ScreenPosition.IsSet())
		{
			DefaultWindowLocation.ScreenPosition = WindowConfig.ScreenPosition.GetValue();
		}
		if (WindowConfig.WindowSize.IsSet())
		{
			DefaultWindowLocation.WindowSize = WindowConfig.WindowSize.GetValue();
		}
		if (WindowConfig.bInitiallyMaximized.IsSet())
		{
			DefaultWindowLocation.InitiallyMaximized = WindowConfig.bInitiallyMaximized.GetValue();
		}

		if (!bShowStartupDialogInPlaceOfMainEditor)
		{
			if( bStartImmersive )
			{
				// Start maximized if we are in immersive mode
				DefaultWindowLocation.InitiallyMaximized = true;
			}

			const bool bIncludeGameName = true;
			WindowConfig.WindowTitle = GetApplicationTitle( bIncludeGameName );
		}

		const FText CloseButtonToolTipText = LOCTEXT("MainFrameModuleCloseButtonToolTip", "Close Unreal Editor");

		// If the parent window is not valid it will be initialized below
		TSharedPtr<SWindow> RootWindowPtr = GetParentWindow();
		if (!RootWindowPtr.IsValid())
		{
			constexpr float MinWidthToAvoidOverlap = 900.f;
			constexpr float MinHeightToAvoidOverlap = 640.f;

			TSharedRef<SWindow> RootWindow = SNew(SWindow)
				.AutoCenter(WindowConfig.CenterRules)
				.Title(WindowConfig.WindowTitle)
				.CloseButtonToolTipText(CloseButtonToolTipText)
				.ScreenPosition(DefaultWindowLocation.ScreenPosition)
				.ClientSize(DefaultWindowLocation.WindowSize)
				.CreateTitleBar(!WindowConfig.bEmbedTitleAreaContent)
				.SizingRule(WindowConfig.bIsUserSizable ? ESizingRule::UserSized : ESizingRule::FixedSize)
				.SupportsMaximize(WindowConfig.bSupportsMaximize)
#if PLATFORM_MAC
				// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
				.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
				.SupportsMinimize(WindowConfig.bSupportsMinimize)
				.MinWidth(MinWidthToAvoidOverlap)
				.MinHeight(MinHeightToAvoidOverlap);

			// Avoid using the is initially maximized argument when creating the window so that the code below can change the window without this initial value taking over taking the changes when the window is first shown.
			if (DefaultWindowLocation.InitiallyMaximized)
			{
				RootWindow->Maximize();
			}

			const bool bShowRootWindowImmediately = false;
			FSlateApplication::Get().AddWindow( RootWindow, bShowRootWindowImmediately );

			FGlobalTabmanager::Get()->SetRootWindow(RootWindow);
			FGlobalTabmanager::Get()->SetAllowWindowMenuBar(true);

			FSlateNotificationManager::Get().SetRootWindow(RootWindow);

			RootWindowPtr = RootWindow;
		}

		// The window has to be valid
		check(RootWindowPtr.IsValid())

		TSharedRef<SWindow> RootWindow = RootWindowPtr.ToSharedRef();

		if (bIsJustCreatingContent)
		{
			if (DefaultWindowLocation.InitiallyMaximized)
			{
				// if initially maximized get the fullscreen size and use that for the size to make the Maximize jump less visible
				// TODO: Should be looking at maximizing without showing the window or to make the InitiallyMaximized work on second window constructs
				DefaultWindowLocation.WindowSize = RootWindow->GetFullScreenInfo().GetSize();
			}
			else if (DefaultWindowLocation.WindowSize.IsNearlyZero())
			{
				// Resize the window with the previous size if not maximized and if the window size was not set (or nearly 0)
				UE::Slate::FDeprecateVector2DResult SizeToRestore = RootWindow->GetSizeInScreen();
				DefaultWindowLocation.WindowSize = SizeToRestore;
			}

			SWindow::FArguments Args;
			Args.AutoCenter(WindowConfig.CenterRules)
				.Title(WindowConfig.WindowTitle)
				.CloseButtonToolTipText(CloseButtonToolTipText)
				.IsInitiallyMaximized(DefaultWindowLocation.InitiallyMaximized)
				.ScreenPosition( DefaultWindowLocation.ScreenPosition)
				.ClientSize(DefaultWindowLocation.WindowSize)
				.CreateTitleBar(!WindowConfig.bEmbedTitleAreaContent)
				.SizingRule(WindowConfig.bIsUserSizable ? ESizingRule::UserSized : ESizingRule::FixedSize)
				.SupportsMaximize(WindowConfig.bSupportsMaximize)
				.SupportsMinimize(WindowConfig.bSupportsMinimize);

			RootWindow->Construct(Args);
			// InitiallyMaximized is not correctly used if the window was already created and visible, so do it manually if requested
			if (DefaultWindowLocation.InitiallyMaximized)
			{
				RootWindow->Maximize();
			}
		}

		bool bLevelEditorIsMainTab = false;
		if (!bShowStartupDialogInPlaceOfMainEditor)
		{
			// Get desktop metrics
			FDisplayMetrics DisplayMetrics;
			FSlateApplication::Get().GetDisplayMetrics( DisplayMetrics );

			const float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint((float)DisplayMetrics.PrimaryDisplayWorkAreaRect.Left, (float)DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

			// Setup a position and size for the main frame window that's centered in the desktop work area
			const float CenterScale = 0.65f;
			const FVector2D DisplaySize(
				DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
				DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top );
			const FVector2D WindowSize = (CenterScale * DisplaySize) / DPIScale;

			// IMPORTANT: If you want to change the default value of "LevelEditor_Layout_v1.1" or "UnrealEd_Layout_v1.6" (even if you only change their version numbers), these are the steps to follow:
			// 1. Check out Engine\Config\Layouts\DefaultLayout.ini in Perforce.
			// 2. Change the code below as you wish and compile the code.
			// 3. (Optional:) Save your current layout so you can load it later.
			// 4. Close the editor.
			// 5. Manually remove Engine\Saved\Config\Layouts\EditorLayout.ini
			// 6. Open the Editor, which will auto-regenerate a default EditorLayout.ini that uses your new code below.
			// 7. "Window" --> "Save Layout" --> "Save Layout As..."
			//     - Name: Default Editor Layout
			//     - Description: Default layout that the Unreal Editor automatically generates
			// 8. Either click on the toast generated by Unreal that would open the saving path or manually open Engine\Saved\Config\Layouts\ in your explorer
			// 9. Move and rename the new file (Engine\Saved\Config\Layouts\Default_Editor_Layout.ini) into Engine\Config\Layouts\DefaultLayout.ini. You might also have to modify:
			//     9.1. QAGame/Config/DefaultEditorLayout.ini
			//     9.2. Engine/Config/BaseEditorLayout.ini
			//     9.3. Etc
			// 10. Push the new "DefaultLayout.ini" together with your new code.
			// 11. Also update these instructions if you change the version number (e.g., from "UnrealEd_Layout_v1.5" to "UnrealEd_Layout_v1.6").
			const FName LayoutName = TEXT("UnrealEd_Layout_v1.6");
			const TSharedRef<FTabManager::FLayout> DefaultLayout =
				// We persist the positioning of the level editor and the content browser.
				// The asset editors currently do not get saved.
				FTabManager::NewLayout(LayoutName)
				->AddArea
				(
					// level editor window
					FTabManager::NewPrimaryArea()
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(2.0f)
						->AddTab("LevelEditor", ETabState::OpenedTab)
						->AddTab("DockedToolkit", ETabState::ClosedTab)
					)
				)
				->AddArea
				(
					// toolkits window
					FTabManager::NewArea(WindowSize)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(1.0f)
						->AddTab("StandaloneToolkit", ETabState::ClosedTab)
					)
				)
				->AddArea
				(
					// settings window
					FTabManager::NewArea(WindowSize)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(1.0f)
						->AddTab("EditorSettings", ETabState::ClosedTab)
						->AddTab("ProjectSettings", ETabState::ClosedTab)
						->AddTab("PluginsEditor", ETabState::ClosedTab)
					)
				);
			const bool bPrimaryAreaMustHaveOpenedTabsToBeValid = true;
			const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::IfNoOpenTabValid;
			TArray<FString> RemovedOlderLayoutVersions;

			const TSharedRef<FTabManager::FLayout> LoadedLayout = bIsReCreatingDefaultLayoutAsFallback
				? DefaultLayout
				: FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, DefaultLayout, OutputCanBeNullptr, RemovedOlderLayoutVersions);


			// If older fields of the layout name (i.e., lower versions than "UnrealEd_Layout_v1.4") were found
			if (RemovedOlderLayoutVersions.Num() > 0)
			{
				// FMessageDialog - Notify the user that the layout version was updated and the current layout uses a deprecated one
				const FText WarningText = FText::Format(LOCTEXT("MainFrameModuleVersionErrorBody", "The expected Unreal Editor layout version is \"{0}\", while only version \"{1}\" was found. I.e., the current layout was created with a previous version of Unreal that is deprecated and no longer compatible.\n\nUnreal will continue with the default layout for its current version, the deprecated one has been removed.\n\nYou can create and save your custom layouts with \"Window\"->\"Save Layout\"->\"Save Layout As...\"."),
					FText::FromString(LayoutName.ToString()), FText::FromString(RemovedOlderLayoutVersions[0]));
				UE_LOGF(LogMainFrame, Warning, "%ls", *WarningText.ToString());
				// If user is trying to load a specific layout with "Load", also warn them with a message dialog
				if (bIsBeingRecreated)
				{
					FMessageDialog::Open(EAppMsgType::Ok, WarningText, LOCTEXT("MainFrameModuleVersionErrorTitle", "Unreal Editor Layout Version Mismatch"));
				}
			}


			FToolMenuContext EmptyContext;
			MakeMainMenu(FGlobalTabmanager::Get(), "MainFrame.NomadMainMenu", EmptyContext);
		
			MainFrameContent = FGlobalTabmanager::Get()->RestoreFrom(LoadedLayout, RootWindow, WindowConfig.bEmbedTitleAreaContent, OutputCanBeNullptr);
			// MainFrameContent will only be nullptr if its main area contains invalid tabs (probably some layout bug). If so, reset layout to avoid potential crashes


			/**
			 * Workaround a compatibility issue that was introduced in the version 1.5
			 * The engine is supposed to be able to reset the layout for that type of accidental issue but that mechanism has been dysfunctional since 2021 and was fixed in 5.8
			 * This fix will be here for an engine a release to help auto fix corrupt config that were created during the window were the issue existed to avoid the crash in the previous release of the engine caused by this issue.
			 */
			{
				static bool bDidPathIssueInPreviousVersion = false;
				if (!bDidPathIssueInPreviousVersion)
				{
					bDidPathIssueInPreviousVersion = true;
					RemovedOlderLayoutVersions.Reset();

					const FName ProblematicLayoutName = TEXT("UnrealEd_Layout_v1.5");
					const TSharedRef<FTabManager::FLayout> Layout =
						// We persist the positioning of the level editor and the content browser.
						// The asset editors currently do not get saved.
						FTabManager::NewLayout(ProblematicLayoutName)
						->AddArea
						(
							// level editor window
							FTabManager::NewPrimaryArea()
							->Split
							(
								FTabManager::NewStack()
								->SetSizeCoefficient(2.0f)
								->AddTab("LevelEditor", ETabState::OpenedTab)
								->AddTab("DockedToolkit", ETabState::ClosedTab)
							)
						)
						->AddArea
						(
							// toolkits window
							FTabManager::NewArea(WindowSize)
							->SetOrientation(Orient_Vertical)
							->Split
							(
								FTabManager::NewStack()
								->SetSizeCoefficient(1.0f)
								->AddTab("StandaloneToolkit", ETabState::ClosedTab)
							)
						)
						->AddArea
						(
							// settings window
							FTabManager::NewArea(WindowSize)
							->Split
							(
								FTabManager::NewStack()
								->SetSizeCoefficient(1.0f)
								->AddTab("EditorSettings", ETabState::ClosedTab)
								->AddTab("ProjectSettings", ETabState::ClosedTab)
								->AddTab("PluginsEditor", ETabState::ClosedTab)
							)
						);


					const TSharedRef<FTabManager::FLayout> LayoutToFix = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout, OutputCanBeNullptr, RemovedOlderLayoutVersions);

					const TSharedPtr<FTabManager::FArea> PrimaryArea = LayoutToFix->GetPrimaryArea().Pin();
					if (PrimaryArea && PrimaryArea->WindowPlacement != FTabManager::FArea::EWindowPlacementInternal::Placement_NoWindow)
					{
						PrimaryArea->WindowPlacement = FTabManager::FArea::EWindowPlacementInternal::Placement_NoWindow;
						FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToFix);
					}
				}
			} 

			if (!MainFrameContent.IsValid())
			{
				// This code will recurse to load the default layout. If that also fail we will get into an infinite loop
				check(!bIsReCreatingDefaultLayoutAsFallback)
				TGuardValue<bool> GuardRecreatingDefaultLayoutFallback(bIsReCreatingDefaultLayoutAsFallback, true);

				TGuardValue<bool> GuardRecreatingDefaultMainFrame(bRecreatingDefaultMainFrame, true);

				// Clean FSlateApplication & FGlobalTabmanager
				FGlobalTabmanager::Get()->CloseAllAreas();
				FSlateApplication::Get().CloseAllWindowsImmediately();

				// Warn user/developer
				UE_LOGF(LogMainFrame, Warning, "UnrealEd layout could not be loaded from the config files, we will use the default provided by the code");
				// Reload default main frame
				CreateDefaultMainFrame(bStartImmersive, bStartPIE);
				return;
			}
			bLevelEditorIsMainTab = true;
		}

		check(MainFrameContent.IsValid());
		RootWindow->SetContent(MainFrameContent.ToSharedRef());

		TSharedPtr<SDockTab> MainTab;
		if ( bLevelEditorIsMainTab )
		{
			MainTab = FGlobalTabmanager::Get()->TryInvokeTab( FTabId("LevelEditor") );

			// make sure we only allow the message log to be shown when we have a level editor main tab
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>(TEXT("MessageLog"));
			MessageLogModule.EnableMessageLogDisplay(!FApp::IsUnattended());
		}

		// Initialize the main frame window
		MainFrameHandler->OnMainFrameGenerated( MainTab, RootWindow );
		
		if (bDelayedShowMainFrame)
		{
			// Setup delegate to show main frame
			DelayedShowMainFrameDelegate.BindLambda([this, RootWindow, bStartImmersive, bStartPIE]()
			{
				// Show the window!
				MainFrameHandler->ShowMainFrameWindow(RootWindow, bStartImmersive, bStartPIE);
			});
		}
		else
		{
			// Show the window!
			MainFrameHandler->ShowMainFrameWindow(RootWindow, bStartImmersive, bStartPIE);
		}
		
		MRUFavoritesList = new FMainMRUFavoritesList;
		MRUFavoritesList->ReadFromINI();

		MainFrameCreationFinishedEvent.Broadcast(RootWindow, bShowStartupDialogInPlaceOfMainEditor);
	}
}

bool FMainFrameModule::IsRecreatingDefaultMainFrame() const
{
	return bRecreatingDefaultMainFrame;
}


TSharedRef<SWidget> FMainFrameModule::MakeMainMenu(const TSharedPtr<FTabManager>& TabManager, const FName MenuName, FToolMenuContext& ToolMenuContext) const
{
	// Can't make the MainMenu without the global editor commands having been registered
	FGlobalEditorCommonCommands::Register();

	return FMainMenu::MakeMainMenu(TabManager, MenuName, ToolMenuContext);
}

TSharedRef<SWidget> FMainFrameModule::MakeDeveloperTools( const TArray<FMainFrameDeveloperTool>& AdditionalTools ) const
{
	struct Local
	{
		static FText GetFrameRateAsString() 
		{
			// Clamp to avoid huge averages at startup or after hitches
			const float AverageFPS = 1.0f / FSlateApplication::Get().GetAverageDeltaTime();
			const float ClampedFPS = ( AverageFPS < 0.0f || AverageFPS > 4000.0f ) ? 0.0f : AverageFPS;

			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(1)
				.SetMaximumFractionalDigits(1);
			return FText::AsNumber( ClampedFPS, &FormatOptions );
		}

		static FText GetFrameTimeAsString() 
		{
			// Clamp to avoid huge averages at startup or after hitches
			const float AverageMS = FSlateApplication::Get().GetAverageDeltaTime() * 1000.0f;
			const float ClampedMS = ( AverageMS < 0.0f || AverageMS > 4000.0f ) ? 0.0f : AverageMS;

			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(1)
				.SetMaximumFractionalDigits(1);
			static const FText FrameTimeFmt = FText::FromString(TEXT("{0} ms"));
			return FText::Format( FrameTimeFmt, FText::AsNumber( ClampedMS, &FormatOptions ) );
		}

		static FText GetMemoryAsString() 
		{
			// Only refresh process memory allocated after every so often, to reduce fixed frame time overhead
			static SIZE_T StaticLastTotalAllocated = 0;
			static int32 QueriesUntilUpdate = 1;
			if( --QueriesUntilUpdate <= 0 )
			{
				// Query OS for process memory used
				FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
				StaticLastTotalAllocated = MemoryStats.UsedPhysical;

				// Wait 60 queries until we refresh memory again
				QueriesUntilUpdate = 60;
			}

			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(2)
				.SetMaximumFractionalDigits(2);
			static const FText MemorySizeFmt = FText::FromString(TEXT("{0} mb"));
			return FText::Format( MemorySizeFmt, FText::AsNumber( (float)StaticLastTotalAllocated / ( 1024.0f * 1024.0f ), &FormatOptions ) );
		}

		static FText GetUObjectCountAsString() 
		{
			return FText::AsNumber(GUObjectArray.GetObjectArrayNumMinusAvailable());
		}

#if STALL_DETECTOR
		static FText GetStallCountAsString()
		{
			return FText::AsNumber(UE::FStallDetectorStats::TotalTriggeredCount.Get());
		}
#endif // STALL_DETECTOR

		static void OpenVideo( FString SourceFilePath )
		{
			FPlatformProcess::ExploreFolder( *( FPaths::GetPath(SourceFilePath) ) );
		}

		/** @return Returns true if frame rate and memory should be displayed in the UI */
		static EVisibility ShouldShowFrameRateAndMemory()
		{
			return GetDefault<UEditorPerformanceSettings>()->bShowFrameRateAndMemory ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
		}

		static void AddSlot(TSharedRef<SHorizontalBox>& HorizontalBox, const FSlateFontInfo& LabelFont, const FSlateFontInfo& ValueFont, const FMainFrameDeveloperTool& DeveloperTool)
		{
			HorizontalBox->AddSlot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(DeveloperTool.Visibility)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					[
						SNew(STextBlock)
						.Text(DeveloperTool.Label)
						.Font(LabelFont)
						.ColorAndOpacity(FLinearColor(0.3f, 0.3f, 0.3f))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					[
						SNew(STextBlock)
						.Text(DeveloperTool.Value)
						.Font(ValueFont)
						.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
					]
				];
		}
	};

	const FSlateFontInfo& SmallFixedFont = FAppStyle::GetFontStyle(TEXT("MainFrame.DebugTools.SmallFont") );
	const FSlateFontInfo& NormalFixedFont = FAppStyle::GetFontStyle(TEXT("MainFrame.DebugTools.NormalFont") );
	const FSlateFontInfo& LabelFont = FAppStyle::GetFontStyle(TEXT("MainFrame.DebugTools.LabelFont") );

	TSharedRef<SHorizontalBox> DeveloperToolWidget =
		SNew( SHorizontalBox )
		.Visibility(GIsDemoMode ? EVisibility::Collapsed : EVisibility::HitTestInvisible);

	for (const FMainFrameDeveloperTool& DeveloperTool : AdditionalTools)
	{
		Local::AddSlot(DeveloperToolWidget, LabelFont, NormalFixedFont, DeveloperTool);
	}

	{
		FMainFrameDeveloperTool FpsDeveloperTool;
		FpsDeveloperTool.Visibility = TAttribute<EVisibility>::Create(&Local::ShouldShowFrameRateAndMemory);
		FpsDeveloperTool.Label = LOCTEXT("FrameRateLabel", "FPS: ");
		FpsDeveloperTool.Value = TAttribute<FText>::Create(&Local::GetFrameRateAsString);
		Local::AddSlot(DeveloperToolWidget, LabelFont, NormalFixedFont, FpsDeveloperTool);
	}
	{
		FMainFrameDeveloperTool FrameDeveloperTool;
		FrameDeveloperTool.Visibility = TAttribute<EVisibility>::Create(&Local::ShouldShowFrameRateAndMemory);
		FrameDeveloperTool.Label = LOCTEXT("FrameRate/FrameTime", "/ ");
		FrameDeveloperTool.Value = TAttribute<FText>::Create(&Local::GetFrameTimeAsString);
		Local::AddSlot(DeveloperToolWidget, LabelFont, NormalFixedFont, FrameDeveloperTool);
	}
	{
		FMainFrameDeveloperTool MemDeveloperTool;
		MemDeveloperTool.Visibility = TAttribute<EVisibility>::Create(&Local::ShouldShowFrameRateAndMemory);
		MemDeveloperTool.Label = LOCTEXT("MemoryLabel", "Mem: ");
		MemDeveloperTool.Value = TAttribute<FText>::Create(&Local::GetMemoryAsString);
		Local::AddSlot(DeveloperToolWidget, LabelFont, NormalFixedFont, MemDeveloperTool);
	}
	{
		FMainFrameDeveloperTool ObjDeveloperTool;
		ObjDeveloperTool.Visibility = TAttribute<EVisibility>::Create(&Local::ShouldShowFrameRateAndMemory);
		ObjDeveloperTool.Label = LOCTEXT("UObjectCountLabel", "Objs: ");
		ObjDeveloperTool.Value = TAttribute<FText>::Create(&Local::GetUObjectCountAsString);
		Local::AddSlot(DeveloperToolWidget, LabelFont, NormalFixedFont, ObjDeveloperTool);
	}
	{
#if STALL_DETECTOR
		FMainFrameDeveloperTool StallsDeveloperTool;
		StallsDeveloperTool.Visibility = TAttribute<EVisibility>::Create(&Local::ShouldShowFrameRateAndMemory);
		StallsDeveloperTool.Label = LOCTEXT("StallsLabel", "Stalls: ");
		StallsDeveloperTool.Value = TAttribute<FText>::Create(&Local::GetStallCountAsString);
		Local::AddSlot(DeveloperToolWidget, LabelFont, NormalFixedFont, StallsDeveloperTool);
#endif // STALL_DETECTOR
	}


	// Invisible border, so that we can animate our box panel size
	return SNew( SBorder )
		.Visibility( EVisibility::SelfHitTestInvisible )
		.Padding( FMargin(0.0f, 0.0f, 0.0f, 1.0f) )
		.VAlign(VAlign_Bottom)
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		[
			SNew( SHorizontalBox )
			.Visibility( EVisibility::SelfHitTestInvisible )

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 0.0f )
			[
				DeveloperToolWidget
			]
		];
}


void FMainFrameModule::SetLevelNameForWindowTitle( const FString& InLevelFileName )
{
	LoadedLevelName = (InLevelFileName.Len() > 0)
		? FPaths::GetBaseFilename(InLevelFileName)
		: NSLOCTEXT("UnrealEd", "Untitled", "Untitled" ).ToString();
}

void FMainFrameModule::SetApplicationTitleOverride(const FText& NewOverriddenApplicationTitle)
{
	OverriddenWindowTitle = NewOverriddenApplicationTitle;
}



/* IModuleInterface implementation
 *****************************************************************************/

void FMainFrameModule::ShowProjectBrowser(bool bAllowProjectOpening, bool bAllowProjectCreate)
{
	if (ShowProjectBrowserOverride.IsBound())
	{
		ShowProjectBrowserOverride.Execute(bAllowProjectOpening, bAllowProjectCreate);
		return;
	}

	FMainFrameActionCallbacks::NewProject(bAllowProjectOpening, bAllowProjectCreate);
}

void FMainFrameModule::StartupModule( )
{
	bDelayedShowMainFrame = false;
	bRecreatingDefaultMainFrame = false;

	MRUFavoritesList = NULL;

	ProjectDialogProvider.Register();
	HomeScreenProvider.Register();

	ensureMsgf(!IsRunningGame(), TEXT("The MainFrame module should only be loaded when running the editor.  Code that extends the editor, adds menu items, etc... should not run when running in -game mode or in a non-WITH_EDITOR build"));
	MainFrameHandler = MakeShareable(new FMainFrameHandler);

	FGenericCommands::Register();
	FMainFrameCommands::Register();

	// Exposes the main frame command list to subscribers from other systems
	FInputBindingManager::Get().RegisterCommandList(FMainFrameCommands::Get().GetContextName(), FMainFrameCommands::ActionList);

	SetLevelNameForWindowTitle(TEXT(""));

	// Register to find out about when hot reload completes, so we can show a notification
	IHotReloadModule& HotReloadModule = IHotReloadModule::Get();
	HotReloadModule.OnModuleCompilerStarted().AddRaw( this, &FMainFrameModule::HandleLevelEditorModuleCompileStarted );
	HotReloadModule.OnModuleCompilerFinished().AddRaw( this, &FMainFrameModule::HandleLevelEditorModuleCompileFinished );
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw( this, &FMainFrameModule::HandleReloadFinished );

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	SourceCodeAccessModule.OnLaunchingCodeAccessor().AddRaw( this, &FMainFrameModule::HandleCodeAccessorLaunching );
	SourceCodeAccessModule.OnDoneLaunchingCodeAccessor().AddRaw( this, &FMainFrameModule::HandleCodeAccessorLaunched );
	SourceCodeAccessModule.OnOpenFileFailed().AddRaw( this, &FMainFrameModule::HandleCodeAccessorOpenFileFailed );

	// HomeScreen Registration
	if (UE::Editor::HomeScreen::IsHomeScreenEnabled())
	{
		FGlobalTabmanager::Get()->RegisterTabSpawner(
			"HomeScreen", FOnSpawnTab::CreateRaw(this, &FMainFrameModule::SpawnHomeScreenWindow))
			.SetAutoGenerateMenuEntry(false)
			.SetMenuType(ETabSpawnerMenuType::Hidden)
			.SetDisplayName(FText::GetEmpty());
    }

	ModuleCompileStartTime = 0.0f;

	// migrate old layout settings
	FLayoutSaveRestore::MigrateConfig(GEditorPerProjectIni, GEditorLayoutIni);
}


void FMainFrameModule::ShutdownModule( )
{
	ClearDelayedShowMainFrameDelegate();

	// Destroy the main frame window
	TSharedPtr< SWindow > ParentWindow( GetParentWindow() );
	if( ParentWindow.IsValid() )
	{
		ParentWindow->DestroyWindowImmediately();
	}

	MainFrameHandler.Reset();

	FMainFrameCommands::Unregister();

	if( IHotReloadModule::IsAvailable() )
	{
		IHotReloadModule& HotReloadModule = IHotReloadModule::Get();
		FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll( this );
		HotReloadModule.OnModuleCompilerStarted().RemoveAll( this );
		HotReloadModule.OnModuleCompilerFinished().RemoveAll( this );
	}

	if(FModuleManager::Get().IsModuleLoaded("SourceCodeAccess"))
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::GetModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		SourceCodeAccessModule.OnLaunchingCodeAccessor().RemoveAll( this );
		SourceCodeAccessModule.OnDoneLaunchingCodeAccessor().RemoveAll( this );
		SourceCodeAccessModule.OnOpenFileFailed().RemoveAll( this );
	}

	// Clean up static object/widget used for the HomeScreen Tab
	HomeScreenTabManager.Reset();

	// Unregister HomeScreen tab on shutdown
	FGlobalTabmanager::Get()->UnregisterTabSpawner(TEXT("HomeScreen"));

	ProjectDialogProvider.UnRegister();
	HomeScreenProvider.UnRegister();
}


void FMainFrameModule::HandleResizeMainFrameCommand(const TArray<FString>& Args)
{
	if (Args.Num() == 2)
	{
		FVector2D Size;
		Size.X = FPlatformString::Atof(*Args[0]);
		Size.Y = FPlatformString::Atof(*Args[1]);

		if (Size.X > 0 && Size.Y > 0)
		{
			FGlobalTabmanager::Get()->GetRootWindow()->ReshapeWindow(FGlobalTabmanager::Get()->GetRootWindow()->GetPositionInScreen(), Size);
		}
	}
	
}

/* FMainFrameModule event handlers
 *****************************************************************************/

void FMainFrameModule::HandleLevelEditorModuleCompileStarted( bool bIsAsyncCompile )
{
	ModuleCompileStartTime = FPlatformTime::Seconds();

	if (CompileNotificationPtr.IsValid())
	{
		CompileNotificationPtr.Pin()->ExpireAndFadeout();
	}

	if ( GEditor )
	{
		GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileStart_Cue.CompileStart_Cue"));
	}

	FNotificationInfo Info( NSLOCTEXT("MainFrame", "RecompileInProgress", "Compiling C++ Code") );
	Info.Image = FAppStyle::GetBrush(TEXT("LevelEditor.RecompileGameCode"));
	Info.ExpireDuration = 5.0f;
	Info.bFireAndForget = false;
	
	// We can only show the cancel button on async builds
	if (bIsAsyncCompile)
	{
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("CancelC++Compilation", "Cancel"), FText(), FSimpleDelegate::CreateRaw(this, &FMainFrameModule::OnCancelCodeCompilationClicked)));
	}

	CompileNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);

	if (CompileNotificationPtr.IsValid())
	{
		CompileNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FMainFrameModule::OnCancelCodeCompilationClicked()
{
	IHotReloadModule::Get().RequestStopCompilation();
}

void FMainFrameModule::HandleLevelEditorModuleCompileFinished(const FString& LogDump, ECompilationResult::Type CompilationResult, bool bShowLog)
{
	// Track stats
	{
		const float ModuleCompileDuration = (float)(FPlatformTime::Seconds() - ModuleCompileStartTime);
		UE_LOGF(LogMainFrame, Log, "MainFrame: Module compiling took %.3f seconds", ModuleCompileDuration);

		if( FEngineAnalytics::IsAvailable() )
		{
			TArray< FAnalyticsEventAttribute > CompileAttribs;
			CompileAttribs.Add(FAnalyticsEventAttribute(TEXT("Duration"), FString::Printf(TEXT("%.3f"), ModuleCompileDuration)));
			CompileAttribs.Add(FAnalyticsEventAttribute(TEXT("Result"), ECompilationResult::ToString(CompilationResult)));
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Modules.Recompile"), CompileAttribs);
		}
	}

	TSharedPtr<SNotificationItem> NotificationItem = CompileNotificationPtr.Pin();

	if (NotificationItem.IsValid())
	{
		if (!ECompilationResult::Failed(CompilationResult))
		{
			if ( GEditor )
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileSuccess_Cue.CompileSuccess_Cue"));
			}

			NotificationItem->SetText(NSLOCTEXT("MainFrame", "RecompileComplete", "Compile Complete!"));
			NotificationItem->SetExpireDuration( 5.0f );
			NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		}
		else
		{
			struct Local
			{
				static void ShowCompileLog()
				{
					FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
					MessageLogModule.OpenMessageLog(FCompilerResultsLog::GetLogName());
				}
			};

			if ( GEditor )
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
			}

			if (CompilationResult == ECompilationResult::FailedDueToHeaderChange)
			{
				NotificationItem->SetText(NSLOCTEXT("MainFrame", "RecompileFailedDueToHeaderChange", "Compile failed due to the header changes. Close the editor and recompile project in IDE to apply changes."));
			}
			else if (CompilationResult == ECompilationResult::Canceled)
			{
				NotificationItem->SetText(NSLOCTEXT("MainFrame", "RecompileCanceled", "Compile Canceled!"));
			}
			else
			{
				NotificationItem->SetText(NSLOCTEXT("MainFrame", "RecompileFailed", "Compile Failed!"));
			}
			
			NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
			NotificationItem->SetHyperlink(FSimpleDelegate::CreateStatic(&Local::ShowCompileLog));
			NotificationItem->SetExpireDuration(30.0f);
		}

		NotificationItem->ExpireAndFadeout();

		CompileNotificationPtr.Reset();
	}
}


void FMainFrameModule::HandleReloadFinished( EReloadCompleteReason Reason )
{
	// Only play the notification for hot reloads that were triggered automatically.  If the user triggered the hot reload, they'll
	// have a different visual cue for that, such as the "Compiling Complete!" notification
	if( Reason == EReloadCompleteReason::HotReloadAutomatic )
	{
		FNotificationInfo Info( LOCTEXT("HotReloadFinished", "Hot Reload Complete!") );
		Info.Image = FAppStyle::GetBrush(TEXT("LevelEditor.RecompileGameCode"));
		Info.FadeInDuration = 0.1f;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 1.5f;
		Info.bUseThrobber = false;
		Info.bUseSuccessFailIcons = true;
		Info.bUseLargeFont = true;
		Info.bFireAndForget = false;
		Info.bAllowThrottleWhenFrameRateIsLow = false;
		auto NotificationItem = FSlateNotificationManager::Get().AddNotification( Info );
		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();
	
		GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileSuccess_Cue.CompileSuccess_Cue"));
	}
}


void FMainFrameModule::HandleCodeAccessorLaunched( const bool WasSuccessful )
{
	TSharedPtr<SNotificationItem> NotificationItem = CodeAccessorNotificationPtr.Pin();
	
	if (NotificationItem.IsValid())
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		const FText AccessorNameText = SourceCodeAccessModule.GetAccessor().GetNameText();

		if (WasSuccessful)
		{
			NotificationItem->SetText( FText::Format(LOCTEXT("CodeAccessorLoadComplete", "{0} loaded!"), AccessorNameText) );
			NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		}
		else
		{
			NotificationItem->SetText( FText::Format(LOCTEXT("CodeAccessorLoadFailed", "{0} failed to launch!"), AccessorNameText) );
			NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
		}

		NotificationItem->ExpireAndFadeout();
		CodeAccessorNotificationPtr.Reset();
	}
}


void FMainFrameModule::HandleCodeAccessorLaunching()
{
	if (CodeAccessorNotificationPtr.IsValid())
	{
		CodeAccessorNotificationPtr.Pin()->ExpireAndFadeout();
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	const FText AccessorNameText = SourceCodeAccessModule.GetAccessor().GetNameText();

	FNotificationInfo Info( FText::Format(LOCTEXT("CodeAccessorLoadInProgress", "Loading {0}"), AccessorNameText) );
	Info.bFireAndForget = false;

	CodeAccessorNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	CodeAccessorNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
}

TSharedRef<SDockTab> FMainFrameModule::SpawnHomeScreenWindow(const FSpawnTabArgs& InArgs)
{
	TSharedRef<SDockTab> HomeScreenTab = SNew(SDockTab)
		.TabRole(ETabRole::MajorTab)
		// Assign a display name that will be used when switching tabs with the ctrl + tab feature to remove this tab from the list
		.Label(IMainFrameModule::GetHomeScreenTabLabel())
		.IconColor(FStyleColors::White.GetSpecifiedColor())
		.ContentPadding(FMargin(0));

	// Always recreate the TabManager and bind it to the new HomeScreen tab, when loading layout for example the OnTabClosed is not called.
	HomeScreenTabManager = FGlobalTabmanager::Get()->NewTabManager(HomeScreenTab);
	HomeScreenTabManager->SetAllowWindowMenuBar(true);
	FToolMenuContext EmptyContext;
	FMainFrameModule::Get().MakeMainMenu(HomeScreenTabManager, "HomeScreen.MainMenu", EmptyContext);
	HomeScreenTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this] (TSharedRef<SDockTab>) { HomeScreenTabManager.Reset(); }));

	HomeScreenTab->SetTabIcon(FAppStyle::Get().GetBrush("HomeScreen.Home"));

	TArray<IHomeScreenProvider*> HomeScreenProviders = IModularFeatures::Get().GetModularFeatureImplementations<IHomeScreenProvider>(IHomeScreenProvider::GetModularFeatureName());		
	IHomeScreenProvider* ActiveProvider = nullptr;
	for (IHomeScreenProvider* Provider : HomeScreenProviders)
	{
		if (Provider && Provider->IsRequestingHomeScreenPageControl())
		{
			if (!ActiveProvider || Provider->GetHomeScreenPriority() > ActiveProvider->GetHomeScreenPriority())
			{
				ActiveProvider = Provider;
			}
		}
	}

	TSharedPtr<SWidget> HomeScreenWidget;
	if (ActiveProvider)
	{
		HomeScreenWidget = CreateHomeScreenWidget(HomeScreenTab, ActiveProvider->GetHomeScreenWidgetSettings());
	}
	else
	{
		HomeScreenWidget = CreateHomeScreenWidget(HomeScreenTab);
	}

	// This will create the necessary object and widgets needed for the HomeScreen and return a newly created one or NullWidget if something went wrong
	HomeScreenTab->SetContent(HomeScreenWidget.ToSharedRef());

	return HomeScreenTab;
}

TSharedRef<SWidget> FMainFrameModule::CreateHomeScreenWidget(const TSharedPtr<SDockTab>& InParentDockTab, FHomeScreenWidgetSettings InSettings)
{
	return SNew(SHomeScreen)
		.SectionLinks(InSettings.SectionLinks)
		.ResourceCommunityEntries(InSettings.ResourcesEntries)
		.AlwaysShowRecentList(InSettings.bAlwaysShowRecentList)
		.RecentProjectList(InSettings.RecentProjectListWidget)
		.UnrealLogoBrush(InSettings.UnrealLogoBrush)
		.OnGetStartupSetting(InSettings.OnGetStartupSetting);
}

void FMainFrameModule::HandleCodeAccessorOpenFileFailed(const FString& Filename)
{
	auto* Info = new FNotificationInfo(FText::Format(LOCTEXT("FileNotFound", "Could not find code file, {0}"), FText::FromString(Filename)));
	Info->ExpireDuration = 3.0f;
	FSlateNotificationManager::Get().QueueNotification(Info);
}

void FMainFrameModule::DestroyRegularAndNonRootWindows(TArray<TSharedRef<SWindow>> InWindowsToDestroy) const
{
	// If the root window was destroyed skip this
	if (IsWindowInitialized())
	{
		// Get the root window
		TSharedRef<SWindow> RootWindow = GetParentWindow().ToSharedRef();
		
		// ::RequestDestroyWindow will remove the window from the TArray so we
		// should iterate over it backwards to make sure that the WindowIndex is still correct.
		for (int32 WindowIndex = InWindowsToDestroy.Num() -1; WindowIndex >= 0; --WindowIndex)
		{
			// Skip the root window
			if (InWindowsToDestroy[WindowIndex] == RootWindow)
			{
				continue;
			}

			// If it is not a regular window skip it since it may be a pop-up window containing a progress bar for example
			const bool bIsRegularWindow = InWindowsToDestroy[WindowIndex]->IsRegularWindow();
			if (!bIsRegularWindow)
			{
				continue;
			}

			InWindowsToDestroy[WindowIndex]->RequestDestroyWindow();
		}
		
		// Destroy the windows immediately from the WindowDestroyQueue
		FSlateApplication::Get().DestroyWindowsImmediately();
	}
}

void FMainFrameModule::RegisterHomeScreenFileMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* FileMenu = ToolMenus->RegisterMenu("HomeScreen.MainMenu.File");

	FToolMenuSection& ProjectSection = FileMenu->AddSection("FileProject", LOCTEXT("ProjectHeading", "Project"));
	ProjectSection.AddMenuEntry(FMainFrameCommands::Get().NewProject);
	ProjectSection.AddMenuEntry(FMainFrameCommands::Get().OpenProject);

#if !PLATFORM_MAC // Handled by app's menu in menu bar
	FToolMenuSection& ExitSection = FileMenu->AddSection("Exit", LOCTEXT("Exit", "Exit"), FToolMenuInsert("FileProject", EToolMenuInsertType::After));
	ExitSection.AddSeparator("Exit");
	ExitSection.AddMenuEntry(FMainFrameCommands::Get().Exit);
#endif
}

void FMainFrameModule::RegisterHomeScreenEditMenu()
{
	UToolMenu* EditMenu = UToolMenus::Get()->RegisterMenu("HomeScreen.MainMenu.Edit");
	FToolMenuSection& Section = EditMenu->AddSection("Configuration", LOCTEXT("ConfigurationHeading", "Configuration"));
		if (GetDefault<UEditorStyleSettings>()->bExpandConfigurationMenus)
		{
			Section.AddSubMenu(
				"EditorPreferencesSubMenu",
				LOCTEXT("EditorPreferencesSubMenuLabel", "Editor Preferences"),
				LOCTEXT("EditorPreferencesSubMenuToolTip", "Configure the behavior and features of this Editor"),
				FNewToolMenuDelegate::CreateStatic(&FSettingsMenu::MakeMenu, FName("Editor")),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon")
			);

			Section.AddSubMenu(
				"ProjectSettingsSubMenu",
				LOCTEXT("ProjectSettingsSubMenuLabel", "Project Settings"),
				LOCTEXT("ProjectSettingsSubMenuToolTip", "Change the settings of the currently loaded project"),
				FNewToolMenuDelegate::CreateStatic(&FSettingsMenu::MakeMenu, FName("Project")),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon")
			);
		}
		else
		{
#if !PLATFORM_MAC // Handled by app's menu in menu bar
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				FName CategoryName;
				FName SectionName;
				MainFrame.GetEditorSettingsDefaultSelectionOverride(CategoryName, SectionName);

				if (CategoryName.IsNone())
				{
					CategoryName = FName("General");
				}
				if (SectionName.IsNone())
				{
					SectionName = FName("Appearance");
				}

				Section.AddMenuEntry(
					"EditorPreferencesMenu",
					LOCTEXT("EditorPreferencesMenuLabel", "Editor Preferences..."),
					LOCTEXT("EditorPreferencesMenuToolTip", "Configure the behavior and features of the Unreal Editor."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon"),
					FUIAction(FExecuteAction::CreateStatic(&FSettingsMenu::OpenSettings, FName("Editor"), CategoryName, SectionName))
				);
			}
#endif //if !PLATFORM_MAC

			Section.AddMenuEntry(
				"ProjectSettingsMenu",
				LOCTEXT("ProjectSettingsMenuLabel", "Project Settings..."),
				LOCTEXT("ProjectSettingsMenuToolTip", "Change the settings of the currently loaded project."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"),
				FUIAction(FExecuteAction::CreateStatic(&FSettingsMenu::OpenSettings, FName("Project"), FName("Project"), FName("General")))
			);
		}
}

void FMainFrameModule::RegisterHomeScreenHelpMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	ToolMenus->RegisterMenu("HomeScreen.MainMenu.Help", "MainFrame.MainMenu.Help", EMultiBoxType::Menu);
}

FDelayedAutoRegisterHelper FMainFrameModule::HomeScreenWindowMenuRegistration(EDelayedRegisterRunPhase::EndOfEngineInit, []{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([] ()
			{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenus* ToolMenus = UToolMenus::Get();
			static const FName HomeScreenMainMenu("HomeScreen.MainMenu");
			if (!ToolMenus->IsMenuRegistered(HomeScreenMainMenu))
			{
				UToolMenu* HomeScreenMenuBar = UToolMenus::Get()->RegisterMenu(HomeScreenMainMenu, NAME_None, EMultiBoxType::MenuBar);

				RegisterHomeScreenFileMenu();
				RegisterHomeScreenEditMenu();
				RegisterHomeScreenHelpMenu();

				HomeScreenMenuBar->StyleName = FName(TEXT("WindowMenuBar"));

				HomeScreenMenuBar->AddSubMenu(
					"HomeScreen",
					NAME_None,
					"File",
					LOCTEXT("FileMenu", "File"),
					LOCTEXT("FileMenu_ToolTip", "Open the file menu")
				);

				HomeScreenMenuBar->AddSubMenu(
					"HomeScreen",
					NAME_None,
					"Edit",
					LOCTEXT("EditMenu", "Edit"),
					LOCTEXT("EditMenu_ToolTip", "Open the edit menu")
					);

				HomeScreenMenuBar->AddSubMenu(
					"HomeScreen",
					NAME_None,
					"Help",
					LOCTEXT("HelpMenu", "Help"),
					LOCTEXT("HelpMenu_ToolTip", "Open the help menu")
					);
			}
		}));
	}
);

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMainFrameModule, MainFrame);
