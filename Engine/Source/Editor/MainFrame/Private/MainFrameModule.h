// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Frame/MainFrameActions.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Interfaces/IMainFrameModule.h"
#include "UnrealEdMisc.h"
#include "Frame/MainFrameHandler.h"
#include "Misc/CompilationResult.h"
#include "Interfaces/IEditorMainFrameProvider.h"
#include "Interfaces/IHomeScreenProvider.h"

/** 
 * Utility class, which hooks into the editor's home screen tab and
 * supply it with its own settings if it is the higher priority provider
 */
class FEditorHomeScreenProvider : public IHomeScreenProvider
{
public:
	void Register();
	void UnRegister();

	//~ Begin IHomeScreenProvider interface
	virtual bool IsRequestingHomeScreenPageControl() const override;
	virtual int32 GetHomeScreenPriority() override;
	virtual FHomeScreenWidgetSettings GetHomeScreenWidgetSettings() const override;
	//~ End IHomeScreenProvider interface
};

/** 
 * Utility class, which hooks into the editor's main window startup and 
 * supplies a project dialog window if the project hasn't been set. 
 */
class FProjectDialogProvider : public IEditorMainFrameProvider
{
public:
	void Register();
	void UnRegister();

	//~ Begin IEditorMainFrameProvider interface
	virtual bool IsRequestingMainFrameControl() const override;
	virtual FMainFrameWindowOverrides GetDesiredWindowConfiguration() const override;
	virtual TSharedRef<SWidget> CreateMainFrameContentWidget() const override;
	//~ End IEditorMainFrameProvider interface
};

/**
 * Editor main frame module
 */
class FMainFrameModule
	: public IMainFrameModule
{
public:

	// IMainFrameModule interface

	virtual void CreateDefaultMainFrame(const bool bStartImmersive, const bool bStartPIE) override;
	virtual void RecreateDefaultMainFrame(const bool bStartImmersive, const bool bStartPIE) override;
	virtual void ReconstructDefaultMainFrameContent(const bool bStartImmersive, const bool bStartPIE) override;
private:
	/**
	 * Shared code between CreateDefaultMainFrame and RecreateDefaultMainFrame
	 * @param bIsBeingRecreated False if it is being called by first time (CreateDefaultMainFrame), and true if it is being recreated (RecreateDefaultMainFrame). If recreated, it will also display 
	 * @param bIsJustCreatingContent False if it is being called by first time (CreateDefaultMainFrame), and true if the content is being reconstructed (ReconstructDefaultMainFrameContent).
	 */
	virtual void CreateDefaultMainFrameAuxiliary(const bool bStartImmersive, const bool bStartPIE, const bool bIsBeingRecreated, const bool bIsJustCreatingContent = false);

public:
	virtual bool IsRecreatingDefaultMainFrame() const override;
	virtual TSharedRef<SWidget> MakeMainMenu(const TSharedPtr<FTabManager>& TabManager, const FName MenuName, FToolMenuContext& ToolMenuContext) const override;
	

	virtual TSharedRef<SWidget> MakeDeveloperTools( const TArray<FMainFrameDeveloperTool>& AdditionalTools ) const override;

	virtual bool IsWindowInitialized( ) const override
	{
		return MainFrameHandler->GetParentWindow().IsValid();
	}
	
	virtual TSharedPtr<SWindow> GetParentWindow( ) const override
	{
		return MainFrameHandler->GetParentWindow();
	}

	virtual void SetMainTab(const TSharedRef<SDockTab>& MainTab) override
	{
		MainFrameHandler->SetMainTab(MainTab);
	}

	virtual void EnableTabClosedDelegate( ) override
	{
		MainFrameHandler->EnableTabClosedDelegate();
	}

	virtual void DisableTabClosedDelegate( ) override
	{
		MainFrameHandler->DisableTabClosedDelegate();
	}

	virtual void RequestCloseEditor( ) override
	{
		ClearDelayedShowMainFrameDelegate();

		if ( MainFrameHandler->CanCloseEditor() )
		{
			MainFrameHandler->ShutDownEditor();
		}
		else
		{
			FUnrealEdMisc::Get().ClearPendingProjectName();
		}
	}

	virtual void SetLevelNameForWindowTitle( const FString& InLevelFileName ) override;
	
	virtual FString GetLoadedLevelName( ) const override
	{ 
		return LoadedLevelName; 
	}

	virtual TSharedRef<FUICommandList>& GetMainFrameCommandBindings( ) override
	{
		return FMainFrameCommands::ActionList;
	}

	virtual class FMainMRUFavoritesList* GetMRUFavoritesList() const override
	{
		return MRUFavoritesList;
	}

	virtual const FText GetApplicationTitle( const bool bIncludeGameName ) const override
	{
		return OverriddenWindowTitle.IsEmpty() ? StaticGetApplicationTitle( bIncludeGameName ) : OverriddenWindowTitle;
	}

	virtual void SetApplicationTitleOverride(const FText& NewOverriddenApplicationTitle) override;

	virtual void ShowAboutWindow( ) const override
	{
		FMainFrameActionCallbacks::AboutUnrealEd_Execute();
	}
	
	DECLARE_DERIVED_EVENT(FMainFrameModule, IMainFrameModule::FMainFrameCreationFinishedEvent, FMainFrameCreationFinishedEvent);
	virtual FMainFrameCreationFinishedEvent& OnMainFrameCreationFinished( ) override
	{
		return MainFrameCreationFinishedEvent;
	}

	DECLARE_DERIVED_EVENT(FMainFrameModule, IMainFrameModule::FMainFrameSDKNotInstalled, FMainFrameSDKNotInstalled);
	virtual FMainFrameSDKNotInstalled& OnMainFrameSDKNotInstalled( ) override
	{
		return MainFrameSDKNotInstalled;
	}
	void BroadcastMainFrameSDKNotInstalled(const FString& PlatformName, const FString& DocLink) override
	{
		return MainFrameSDKNotInstalled.Broadcast(PlatformName, DocLink);
	}

	DECLARE_DERIVED_EVENT(FMainFrameModule, IMainFrameModule::FMainFrameRequestResource, FMainFrameRequestResource);
	virtual FMainFrameRequestResource& OnMainFrameRequestResource() override
	{
		return MainFrameRequestResource;
	}
	void BroadcastMainFrameRequestResource(const FString& Category, const FString& ResourceName) override
	{
		return MainFrameRequestResource.Broadcast(Category, ResourceName);
	}

	virtual void EnableDelayedShowMainFrame() override
	{
		bDelayedShowMainFrame = true;
	}

	virtual void ShowDelayedMainFrame() override
	{
		bDelayedShowMainFrame = false;

		if (DelayedShowMainFrameDelegate.IsBound())
		{
			DelayedShowMainFrameDelegate.Execute();
			ClearDelayedShowMainFrameDelegate();
		}
	}
	
	virtual FDelegateHandle RegisterCanCloseEditor(const FMainFrameCanCloseEditor& InDelegate) override
	{
		return CanCloseEditorDelegates.Add_GetRef(InDelegate).GetHandle();
	}

	virtual void UnregisterCanCloseEditor(FDelegateHandle InHandle) override
	{
		CanCloseEditorDelegates.RemoveAll([InHandle](const FMainFrameCanCloseEditor& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual bool ExecuteCanCloseEditorDelegates() override
	{
		for (const FMainFrameCanCloseEditor& It : CanCloseEditorDelegates)
		{
			if (It.IsBound() && !It.Execute())
			{
				return false;
			}
		}

		return true;
	}

	virtual void SetEditorSettingsDefaultSelectionOverride(FName CategoryName = FName(), FName SectionName = FName()) override
	{
		EditorSettingsDefaultCategoryOverride = CategoryName;
		EditorSettingsDefaultSectionOverride = SectionName;
	}

	virtual void GetEditorSettingsDefaultSelectionOverride(FName& OutCategoryName, FName& OutSectionName) override
	{
		OutCategoryName = EditorSettingsDefaultCategoryOverride;
		OutSectionName = EditorSettingsDefaultSectionOverride;
	}

	// Opens the project browser in a new window
	virtual void ShowProjectBrowser(bool bAllowProjectOpening, bool bAllowProjectCreate) override;

	// Overrides the functionality of ShowProjectBrowser with the provided delegate
	virtual FDelegateHandle RegisterShowProjectBrowserOverride(const FShowProjectBrowser& InShowProjectBrowserOverride) override
	{
		ShowProjectBrowserOverride = InShowProjectBrowserOverride;
		return ShowProjectBrowserOverride.GetHandle();
	}

	// Removes the override of ShowProjectBrowser if it matches the handle
	virtual void UnregisterShowProjectBrowserOverride(FDelegateHandle InHandle) override
	{
		if (ShowProjectBrowserOverride.GetHandle() == InHandle)
		{
			ShowProjectBrowserOverride = FShowProjectBrowser{};
		}
	}

	// IModuleInterface interface

	virtual void StartupModule( ) override;
	virtual void ShutdownModule( ) override;

	virtual bool SupportsDynamicReloading( ) override
	{
		return true; // @todo: Eventually, this should probably not be allowed.
	}

	static void HandleResizeMainFrameCommand(const TArray<FString>& Args);

public:

	/** Get the size of the project browser window */
	static FVector2D GetProjectBrowserWindowSize() { return FVector2D(1206, 822); }

	// Create the home screen widget
	virtual TSharedRef<SWidget> CreateHomeScreenWidget(const TSharedPtr<SDockTab>& InParentTab, FHomeScreenWidgetSettings InSettings = FHomeScreenWidgetSettings()) override;

private:

	// Spawns the home screen tab
	TSharedRef<SDockTab> SpawnHomeScreenWindow(const FSpawnTabArgs& InArgs);

	// Handles the level editor module starting to recompile.
	void HandleLevelEditorModuleCompileStarted( bool bIsAsyncCompile );

	// Handles the user requesting the current compilation to be canceled
	void OnCancelCodeCompilationClicked();

	// Handles the level editor module finishing to recompile.
	void HandleLevelEditorModuleCompileFinished( const FString& LogDump, ECompilationResult::Type CompilationResult, bool bShowLog );

	/** Called when Reload completes */
	void HandleReloadFinished( EReloadCompleteReason Reason );

	// Handles the code accessor having finished launching its editor
	void HandleCodeAccessorLaunched( const bool WasSuccessful );

	// Handle an open file operation failing
	void HandleCodeAccessorOpenFileFailed(const FString& Filename);

	// Handles launching code accessor
	void HandleCodeAccessorLaunching( );

	// Reset delegate
	void ClearDelayedShowMainFrameDelegate()
	{
		DelayedShowMainFrameDelegate.Unbind();
	}

	// Destroy the given array of windows if they are regular windows and not the root one
	void DestroyRegularAndNonRootWindows(TArray<TSharedRef<SWindow>> InWindowsToDestroy) const;

	// Register HomeScreen file window menu
	static void RegisterHomeScreenFileMenu();

	// Register HomeScreen edit window menu
	static void RegisterHomeScreenEditMenu();

	// Register HomeScreen help window menu
	static void RegisterHomeScreenHelpMenu();

	// Register the Window Menu for the HomeScreen tab
    static FDelayedAutoRegisterHelper HomeScreenWindowMenuRegistration;
private:

	// Weak pointer to the level editor's compile notification item.
	TWeakPtr<SNotificationItem> CompileNotificationPtr;

	// Friendly name for persistently level name currently loaded.  Used for window and tab titles.
	FString LoadedLevelName;

	// Override window title, or empty to not override
	FText OverriddenWindowTitle;

	// Overrides the category that gets selected by default when opening editor settings
	FName EditorSettingsDefaultCategoryOverride;

	// Overrides the section that gets selected by default when editor settings
	FName EditorSettingsDefaultSectionOverride;

	/// Event to be called when the mainframe is fully created.
	FMainFrameCreationFinishedEvent MainFrameCreationFinishedEvent;

	/// Event to be called when the editor tried to use a platform, but it wasn't installed
	FMainFrameSDKNotInstalled MainFrameSDKNotInstalled;

	/// Event to be called to make an open-ended request for a resource from any registered listeners
	FMainFrameRequestResource MainFrameRequestResource;

	// Commands used by main frame in menus and key bindings.
	TSharedPtr<class FMainFrameCommands> MainFrameActions;

	// Holds the main frame handler.
	TSharedPtr<class FMainFrameHandler> MainFrameHandler;

	// Absolute real time that we started compiling modules. Used for stats tracking.
	double ModuleCompileStartTime;

	// Holds the collection of most recently used favorites.
	class FMainMRUFavoritesList* MRUFavoritesList;

	// Weak pointer to the code accessor's notification item.
	TWeakPtr<class SNotificationItem> CodeAccessorNotificationPtr;

	// Delegate that holds a delayed call to ShowMainFrameWindow
	FSimpleDelegate DelayedShowMainFrameDelegate;
	
	// List of delegates that are called after user requests the editor to close that can stop the close
	TArray<FMainFrameCanCloseEditor> CanCloseEditorDelegates;

	// Allow delaying when to show main frame's window
	bool bDelayedShowMainFrame;

	// Is recreating Default Main Frame
	bool bRecreatingDefaultMainFrame;

	// Instantiation of the object responsible for spawning the editor's project dialog on startup
	FProjectDialogProvider ProjectDialogProvider;

	// Instantiation of the object responsible for getting the home screen settings to use if active
	FEditorHomeScreenProvider HomeScreenProvider;

	// Set to true if we are in the middle of recreating the default layout because a previous load attempt failed
	bool bIsReCreatingDefaultLayoutAsFallback = false;

	// Holds the tab manager that manages the HomeScreen tab.
	TSharedPtr<FTabManager> HomeScreenTabManager = nullptr;

	// Stores the delegate for the override of ShowProjectBrowser, if any
	FShowProjectBrowser ShowProjectBrowserOverride;
};
