// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Menus/LayoutsMenu.h"
// Runtime/Core
#include "Containers/Array.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/FileManagerGeneric.h"
#include "Logging/MessageLog.h"
#include "Templates/SharedPointer.h"
// Runtime
#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
// Developer
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "ToolMenus.h"
// Editor
#include "Settings/EditorStyleSettings.h"
#include "Dialog/SCustomDialog.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Frame/MainFrameActions.h"
#include "LevelViewportActions.h"
#include "Menus/SaveLayoutDialog.h"
#include "Misc/MessageDialog.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UnrealEdGlobals.h"
#include "UnrealEdMisc.h"

#define LOCTEXT_NAMESPACE "MainFrameActions"

DEFINE_LOG_CATEGORY_STATIC(LogLayoutsMenu, Fatal, All);



/* FPrivateLayoutsMenu definition
 *****************************************************************************/

class FPrivateLayoutsMenu
{
public:
	enum class ELayoutsMenu
	{
		Load,
		Save,
		Remove
	};

	/** Get LayoutsDirectory path */
	static FString CreateAndGetDefaultLayoutDirInternal(const FLayoutsMenu::ELayoutsType InLayoutsType);

	static TArray<FString> GetIniFilesInFolderInternal(const FString& InStringDirectory);

	static bool TrySaveLayoutOrWarnInternal(const FString& InSourceFilePath, const FString& InTargetFilePath, const FText& InWhatIsThis,
		const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues, const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = false, const bool bShowSaveToast = false);

	/** Name into display text */
	static FText GetDisplayTextInternal(const FString& InString);
	static FText GetTooltipTextInternal(const FText& InDisplayName, const FString& InLayoutFilePath, const FText& InLayoutName);

	static void DisplayLayoutsInternal(FToolMenuSection& InSection, const TArray<FString>& InLayoutIniFileNames, const FString& InLayoutsDirectory, const ELayoutsMenu InLayoutsMenu,
		const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * GetOriginalEditorLayoutIniFilePathInternal() and GetDuplicatedEditorLayoutIniFilePathInternal() are used because sometimes the layout saved is not the same than the one loaded, even
	 * though the visual display and screenshot are 100% the same. In those cases, we still want to show the check mark in the load/save/remove menu indicating that the layout is the same
	 * than the one in the loaded ini file, even though the actual files might not be exactly the same. In addition, this also helps when temporarily closing unrecognized tabs (i.e., the
	 * ones that FTabManager::SpawnTab cannot recognized and keeps closed but still saves them in the layout).
	 */
	static const FString& GetOriginalEditorLayoutIniFilePathInternal();

	static const FString& GetDuplicatedEditorLayoutIniFilePathInternal();

	static bool AreFilesIdenticalInternal(const FString& InFirstFileFullPath, const FString& InSecondFileFullPath);

	static void RemoveTempEditorLayoutIniFilesInternal();

	static bool IsLayoutCheckedInternal(const FString& InLayoutFullPath, const bool bCheckTempFileToo);

	static FText GetProjectLayoutSectionName();

	static void MakeXLayoutsMenuInternal(UToolMenu* InToolMenu, const ELayoutsMenu InLayoutsMenu);

	/** Checks and returns whether the selected layout can be read (e.g., when loading layouts). */
	static bool CanChooseLayoutWhenReadInternal();

	/** Checks and returns whether the selected layout can be modified (e.g., when overriding or removing layouts). */
	static bool CanChooseLayoutWhenWriteInternal(const FLayoutsMenu::ELayoutsType InLayoutsType);

	static void SaveLayoutWithoutRemovingTempLayoutFiles();

	/**
	 * It simply checks whether PIE, SIE, or any Asset Editor is opened, and asks the user whether they want to continue closing them or cancel the Editor layout load
	 * @return Whether we should continue loading the layout
	 */
	static bool CheckAskUserToClosePIESIE(const FText& InitialMessage);

	static void SaveExportLayoutCommon(const FString& InDefaultDirectory, const bool bMustBeSavedInDefaultDirectory, const FText& InWhatIsThis,
		const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields);

	static int32 GetNumberLayoutFiles(const FString& InLayoutsDirectory);
};



/* FPrivateLayoutsMenu public functions
 *****************************************************************************/

FString FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	// Get LayoutsDirectory path
	const FString LayoutsDirectory = (
		InLayoutsType == FLayoutsMenu::ELayoutsType::Engine ? FPaths::EngineDefaultLayoutDir() :
		InLayoutsType == FLayoutsMenu::ELayoutsType::Project ? FPaths::EngineProjectLayoutDir() :
		InLayoutsType == FLayoutsMenu::ELayoutsType::User ? FPaths::EngineUserLayoutDir() :
		TEXT(""));
	checkf(LayoutsDirectory.Len() > 0, TEXT("LayoutsDirectory.Len() was 0 with LayoutType = %d"), int32(InLayoutsType));
	// If the directory does not exist, create it (but it will not have saved Layouts inside)
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*LayoutsDirectory))
	{
		PlatformFile.CreateDirectory(*LayoutsDirectory);
	}
	// Return result
	return LayoutsDirectory;
}

TArray<FString> FPrivateLayoutsMenu::GetIniFilesInFolderInternal(const FString& InStringDirectory)
{
	// Find all ini files in folder
	TArray<FString> LayoutIniFileNames;
	const FString LayoutIniFilePaths = FPaths::Combine(*InStringDirectory, TEXT("*.ini"));
	FFileManagerGeneric::Get().FindFiles(LayoutIniFileNames, *LayoutIniFilePaths, true, false);
	return LayoutIniFileNames;
}

bool FPrivateLayoutsMenu::TrySaveLayoutOrWarnInternal(const FString& InSourceFilePath, const FString& InTargetFilePath, const FText& InWhatIsThis,
	const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues, const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields, const bool bShowSaveToast)
{
	// We must re-read configs to avoid the Editor using a previously cached version
	GConfig->UnloadFile(InSourceFilePath); 
	GConfig->UnloadFile(InTargetFilePath);

	bool bCleanLayoutNameAndDescriptionFields = false;

	// If we are checking whether to clean the fields, we only want to maintain them if we are saving the file into an existing file that already has the same field values
	if (bCleanLayoutNameAndDescriptionFieldsIfNoSameValues)
	{
		// read the source name and description
		const FText LayoutNameSource = FLayoutSaveRestore::LoadSectionFromConfig(InSourceFilePath, "LayoutName");
		const FText LayoutDescriptionSource = FLayoutSaveRestore::LoadSectionFromConfig(InSourceFilePath, "LayoutDescription");

		// read the target name and description
		const FText LayoutNameTarget = FLayoutSaveRestore::LoadSectionFromConfig(InTargetFilePath, "LayoutName");
		const FText LayoutDescriptionTarget = FLayoutSaveRestore::LoadSectionFromConfig(InTargetFilePath, "LayoutDescription");

		// The output target exists (overriding)
		// These fields are not empty in source
		if (!LayoutNameSource.IsEmpty() || !LayoutDescriptionSource.IsEmpty())
		{
			// These fields are different than the ones in target
			if ((LayoutNameSource.ToString() != LayoutNameTarget.ToString()) || (LayoutDescriptionSource.ToString() != LayoutDescriptionTarget.ToString()))
			{
				bCleanLayoutNameAndDescriptionFields = true;
				// We should clean the layout name and description fields, but ask user first
				if (bShouldAskBeforeCleaningLayoutNameAndDescriptionFields)
				{
					// Open Dialog
					const FText TextTitle = LOCTEXT("OverrideLayoutNameAndDescriptionFieldBodyTitle", "Preserve UI Layout Name and Description Fields?");
					const FText TextBody = FText::Format(
						LOCTEXT("OverrideLayoutNameAndDescriptionFieldBody", "You are saving a layout that contains a custom layout name and/or description. Do you also want to copy these 2 properties?\n - Current layout name: {0}\n - Current layout description: {1}\n\nIf you select \"Preserve Values\", the displayed name and description of the original layout customization will also be copied into the new configuration file.\n\nIf you select \"Clear Values\", these fields will be emptied.\n\nIf you are not sure, select \"Preserve Values\" if you are exporting the layout configuration without making any changes, or \"Clear Values\" if you"" have made or plan to make changes to the layout.\n\n"),
						LayoutNameSource, LayoutDescriptionSource);

					// Dialog SWidget
					TSharedRef<SVerticalBox> DialogContents = SNew(SVerticalBox);
					DialogContents->AddSlot()
						.Padding(0, 16, 0, 0)
						[
							SNew(STextBlock)
							.Text(TextBody)
						];

					const FText PreserveValuesText = LOCTEXT("PreserveValuesText", "Preserve Values");
					const FText ClearValuesText = LOCTEXT("ClearValuesText", "Clear Values");
					const FText CancelText = NSLOCTEXT("Dialogs", "EAppReturnTypeCancel", "Cancel");
					TSharedRef<SCustomDialog> CustomDialog = SNew(SCustomDialog)
						.Title(TextTitle)
						.Content()
						[
							DialogContents
						]
						.Buttons({ SCustomDialog::FButton(PreserveValuesText), SCustomDialog::FButton(ClearValuesText), SCustomDialog::FButton(CancelText) });

					// Returns 0 when "Preserve Values" is pressed, 1 when "Clear Values" is pressed, or 2 when Cancel/Esc is pressed
					const int32 ButtonPressed = CustomDialog->ShowModal();
					// Preserve Values
					if (ButtonPressed == 0)
					{
						bCleanLayoutNameAndDescriptionFields = false;
					}
					// Clear Values
					else if (ButtonPressed == 1)
					{
						bCleanLayoutNameAndDescriptionFields = true;
					}
					// Cancel or Esc or window closed
					else if (ButtonPressed == 2 || ButtonPressed == -1)
					{
						return false;
					}
					// This should never occur
					else
					{
						ensureMsgf(false, TEXT("This option should never occur, something went wrong!"));
						return false;
					}
				}
			}
		}
	}

	const FString TargetAbsoluteFilePath = FPaths::ConvertRelativePathToFull(InTargetFilePath);
	if (!FLayoutSaveRestore::DuplicateConfig(InSourceFilePath, InTargetFilePath))
	{
		const FText SourceAbsoluteFilePathText = FText::FromString(FPaths::ConvertRelativePathToFull(InSourceFilePath));		
		const FText TargetAbsoluteFilePathText = FText::FromString(TargetAbsoluteFilePath);

		FMessageLog EditorErrors("EditorErrors");
		FText TextBody;
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("WhatIs"), InWhatIsThis);

		// Source does not exist
		if (!FPaths::FileExists(InSourceFilePath))
		{
			Arguments.Add(TEXT("FileName"), SourceAbsoluteFilePathText);
			TextBody = FText::Format(LOCTEXT("UnsuccessfulSave_NoExist_Notification", "The requested operation ({WhatIs}) was unsuccessful, the desired file does not exist. File path:\n{FileName}"), Arguments);
			EditorErrors.Warning(TextBody);
		}
		// Target is read-only
		else if (IFileManager::Get().IsReadOnly(*InTargetFilePath))
		{
			Arguments.Add(TEXT("FileName"), TargetAbsoluteFilePathText);
			TextBody = FText::Format(LOCTEXT("UnsuccessfulSave_ReadOnly_Notification", "The requested operation ({WhatIs}) was unsuccessful, the target file path is read-only. File path:\n{FileName}"), Arguments);
			EditorErrors.Warning(TextBody);
		}
		// Target and source are the same
		else if (TargetAbsoluteFilePathText.EqualTo(SourceAbsoluteFilePathText))
		{
			Arguments.Add(TEXT("SourceFileName"), SourceAbsoluteFilePathText);
			Arguments.Add(TEXT("FinalFileName"), TargetAbsoluteFilePathText);
			TextBody = FText::Format(LOCTEXT("UnsuccessfulSave_Identical_Notification", "The requested operation ({WhatIs}) was unsuccessful, target and source layout file paths are the same ({SourceFileName})!\nAre you trying to import or replace a file that is already in the layouts folder? If so, remove the current file first."), Arguments);
			EditorErrors.Warning(TextBody);
		}
		// We don't specifically know why it failed, this is a fallback
		else
		{
			Arguments.Add(TEXT("SourceFileName"), SourceAbsoluteFilePathText);
			Arguments.Add(TEXT("FinalFileName"), TargetAbsoluteFilePathText);
			TextBody = FText::Format(LOCTEXT("UnsuccessfulSave_Fallback_Notification", "The requested operation ({WhatIs}) was unsuccessful while copying the layout file from\n{SourceFileName}\ninto\n{FinalFileName}\n\nUsually, this occurs when the introduced file name contains unsupported characters or the total path length exceeds the OS limit."), Arguments);
			EditorErrors.Warning(TextBody);
		}
		EditorErrors.Notify(LOCTEXT("LoadUnsuccessful_Title", "Load Unsuccessful!"));

		// Show reason
		FMessageDialog::Open(EAppMsgType::Ok, TextBody, LOCTEXT("UnsuccessfulCopyHeader", "Unsuccessful copy!"));

		// Return
		return false;
	}
	// Copy successful
	else
	{
		// Clean Layout Name and Description fields
		// We copy twice to make sure we can copy.
		// Problem if we only copied once: If the copy fails, the current EditorLayout.ini would be modified and no longer matches the previous one.
		// The ini file should only be modified if it has been successfully copied to the new (and modified) INI file.
		if (bCleanLayoutNameAndDescriptionFields)
		{
			// Update fields
			FLayoutSaveRestore::SaveSectionToConfig(GEditorLayoutIni, "LayoutName", TEXT(""));
			FLayoutSaveRestore::SaveSectionToConfig(GEditorLayoutIni, "LayoutDescription", TEXT(""));

			// Flush file
			const bool bRead = true;
			GConfig->Flush(bRead, GEditorLayoutIni);

			// Re-copy file
			if (TargetAbsoluteFilePath != FPaths::ConvertRelativePathToFull(GEditorLayoutIni))
			{
				const bool bShouldReplace = true;
				const bool bCopyEvenIfReadOnly = true;
				const bool bCopyAttributes = false;
				IFileManager::Get().Copy(*TargetAbsoluteFilePath, *GEditorLayoutIni, bShouldReplace, bCopyEvenIfReadOnly, bCopyAttributes);
			}
		}

		// Display Editor toast to inform the user of the result of the operation
		if (bShowSaveToast)
		{
			// Code copied from EditorViewportClient.cpp --> FEditorViewportClient::TakeScreenshot(...) to maintain same format than when saving a screenshot
			FNotificationInfo Info(FText::GetEmpty());
			Info.ExpireDuration = 5.0f;
			Info.bUseSuccessFailIcons = false;
			Info.bUseLargeFont = false;

			TSharedPtr<SNotificationItem> SaveMessagePtr = FSlateNotificationManager::Get().AddNotification(Info);
			if (SaveMessagePtr.IsValid())
			{
				const FString HyperLinkString = TargetAbsoluteFilePath;
				auto OpenScreenshotFolder = [HyperLinkString]
				{
					FPlatformProcess::ExploreFolder(*HyperLinkString);
				};
				SaveMessagePtr->SetText(LOCTEXT("SuccessfulSave_Toast", "Editor layout file saved as"));
				SaveMessagePtr->SetHyperlink(FSimpleDelegate::CreateLambda(OpenScreenshotFolder), FText::FromString(HyperLinkString));
				SaveMessagePtr->SetCompletionState(SNotificationItem::CS_Success);
			}
		}
	}

	return true;
}

FText FPrivateLayoutsMenu::GetDisplayTextInternal(const FString& InString)
{
	const FText DisplayNameText = FText::FromString(FPaths::GetBaseFilename(*InString));
	const bool bIsBool = false;
	const FText DisplayName = FText::FromString(FName::NameToDisplayString(DisplayNameText.ToString(), bIsBool));
	return DisplayName;
}

FText FPrivateLayoutsMenu::GetTooltipTextInternal(const FText& InDisplayName, const FString& InLayoutFilePath, const FText& InLayoutName)
{
	FText Tooltip;
	if (InLayoutName.IsEmpty())
	{
		Tooltip = FText::Format(LOCTEXT("DisplayNameFmt", "Layout name:\n{0}\n\nFull file path:\n{1}"), InDisplayName, FText::FromString(InLayoutFilePath));
	}
	else
	{
		Tooltip = FText::Format(LOCTEXT("LayoutNameFmt", "Description:\n{0}\n\nFull file path:\n{1}"), InLayoutName, FText::FromString(InLayoutFilePath));
	}
	return Tooltip;
}

void FPrivateLayoutsMenu::DisplayLayoutsInternal(FToolMenuSection& InSection, const TArray<FString>& InLayoutIniFileNames, const FString& InLayoutsDirectory,
	const ELayoutsMenu InLayoutsMenu, const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	// If there are Layout ini files, read them
	for (int32 LayoutIndex = 0; LayoutIndex < InLayoutIniFileNames.Num(); ++LayoutIndex)
	{
		FString LayoutFilePath = FConfigCacheIni::NormalizeConfigIniPath(*FPaths::Combine(InLayoutsDirectory, InLayoutIniFileNames[LayoutIndex]));

		// Make sure it is a layout file
		GConfig->UnloadFile(LayoutFilePath); // We must re-read it to avoid the Editor to use a previously cached name and description
		if (FLayoutSaveRestore::IsValidConfig(LayoutFilePath))
		{
			// Read and display localization name from INI file
			constexpr bool bAreSectionsOptional = true;
			const FText LayoutName = FLayoutSaveRestore::LoadSectionFromConfig(LayoutFilePath, "LayoutName", bAreSectionsOptional);
			const FText LayoutDescription = FLayoutSaveRestore::LoadSectionFromConfig(LayoutFilePath, "LayoutDescription", bAreSectionsOptional);
			// If no localization name, then display the file name
			const FText DisplayName = (!LayoutName.IsEmpty() ? LayoutName : GetDisplayTextInternal(InLayoutIniFileNames[LayoutIndex]));
			const FText Tooltip = GetTooltipTextInternal(DisplayName, LayoutFilePath, LayoutDescription);

			// Create UI action here that calls the necessary code in FLayoutsMenuLoad, Save, or Remove 
			FUIAction UIAction;
			if (InLayoutsMenu == ELayoutsMenu::Load)
			{
				UIAction = FUIAction(FExecuteAction::CreateStatic(&FLayoutsMenuLoad::LoadLayout, LayoutIndex, InLayoutsType));
			}
			else if (InLayoutsMenu == ELayoutsMenu::Save)
			{
				if (FPrivateLayoutsMenu::CanChooseLayoutWhenWriteInternal(InLayoutsType))
				{
					UIAction = FUIAction(FExecuteAction::CreateStatic(&FLayoutsMenuSave::OverrideLayout, LayoutIndex, InLayoutsType));
				}
				else
				{
					// Cannot save engine or project layouts
					continue;
				}
			}
			else if (InLayoutsMenu == ELayoutsMenu::Remove)
			{
				if (FPrivateLayoutsMenu::CanChooseLayoutWhenWriteInternal(InLayoutsType))
				{
					UIAction = FUIAction(FExecuteAction::CreateStatic(&FLayoutsMenuRemove::RemoveLayout, LayoutIndex, InLayoutsType));
				}
				else
				{
					// Cannot delete engine or project layouts
					continue;
				}
			}
			else
			{
				ensureMsgf(false, TEXT("InLayoutsMenu = %d."), int32(InLayoutsMenu));
			}
			// Run desired action
			InSection.AddMenuEntry(NAME_None, DisplayName, Tooltip, FSlateIcon(), FToolUIActionChoice(UIAction));
		}
	}
}

const FString& FPrivateLayoutsMenu::GetOriginalEditorLayoutIniFilePathInternal()
{
	static const FString OriginalEditorLayoutIniFilePath = GEditorLayoutIni + TEXT("_orig.ini");
	return OriginalEditorLayoutIniFilePath;
}

const FString& FPrivateLayoutsMenu::GetDuplicatedEditorLayoutIniFilePathInternal()
{
	static const FString DuplicatedEditorLayoutIniFilePath = GEditorLayoutIni + TEXT("_temp.ini");
	return DuplicatedEditorLayoutIniFilePath;
}

bool FPrivateLayoutsMenu::AreFilesIdenticalInternal(const FString& InFirstFileFullPath, const FString& InSecondFileFullPath)
{
	// Checked if same file. I.e.,
		// 1. Same size
		// 2. And same internal text
	const bool bHaveSameSize = (IFileManager::Get().FileSize(*InFirstFileFullPath) == IFileManager::Get().FileSize(*InSecondFileFullPath));
	// Same size --> Same layout file?
	if (bHaveSameSize)
	{
		// Read files and check whether they have the exact same text
		FString StringFirstFileFullPath;
		FFileHelper::LoadFileToString(StringFirstFileFullPath, *InFirstFileFullPath);
		FString StringSecondFileFullPath;
		FFileHelper::LoadFileToString(StringSecondFileFullPath, *InSecondFileFullPath);
		// (No) same text = (No) same layout file
		return (StringFirstFileFullPath == StringSecondFileFullPath);
	}
	// No same size = No same layout file
	else
	{
		return false;
	}
}

void FPrivateLayoutsMenu::RemoveTempEditorLayoutIniFilesInternal()
{
	const bool bRequireExists = false;
	const bool bEvenIfReadOnly = true;
	const bool bIsQuiet = false;
	// DuplicatedEditorLayoutIniFilePath
	const FString& DuplicatedEditorLayoutIniFilePath = GetDuplicatedEditorLayoutIniFilePathInternal();
	IFileManager::Get().Delete(*DuplicatedEditorLayoutIniFilePath, bRequireExists, bEvenIfReadOnly, bIsQuiet);
	GConfig->UnloadFile(DuplicatedEditorLayoutIniFilePath);
	// OriginalEditorLayoutIniFilePath
	const FString& OriginalEditorLayoutIniFilePath = GetOriginalEditorLayoutIniFilePathInternal();
	IFileManager::Get().Delete(*OriginalEditorLayoutIniFilePath, bRequireExists, bEvenIfReadOnly, bIsQuiet);
	GConfig->UnloadFile(OriginalEditorLayoutIniFilePath);
}

bool FPrivateLayoutsMenu::IsLayoutCheckedInternal(const FString& InLayoutFullPath, const bool bCheckTempFileToo)
{
	// If same file, return true
	if (AreFilesIdenticalInternal(InLayoutFullPath, GEditorLayoutIni))
	{
		return true;
	}
	// No same size, check if same than temporary one
	else if (bCheckTempFileToo)
	{
		const FString& OriginalEditorLayoutIniFilePath = GetOriginalEditorLayoutIniFilePathInternal();
		const FString& DuplicatedEditorLayoutIniFilePath = GetDuplicatedEditorLayoutIniFilePathInternal();
		if (AreFilesIdenticalInternal(GEditorLayoutIni, DuplicatedEditorLayoutIniFilePath))
		{
			return AreFilesIdenticalInternal(InLayoutFullPath, OriginalEditorLayoutIniFilePath);
		}
		// If GEditorLayoutIni != DuplicatedEditorLayoutIniFilePath, remove DuplicatedEditorLayoutIniFilePath & OriginalEditorLayoutIniFilePath
		else
		{
			RemoveTempEditorLayoutIniFilesInternal();
			return false;
		}
	}
	// No same size, and we should not check if same than temporary ones, so then it is false
	else
	{
		return false;
	}
}

FText FPrivateLayoutsMenu::GetProjectLayoutSectionName()
{
	const FString ProjectDir = FPaths::ProjectDir(); // "../../../QAGame/"
	const FString ProjectDirName = FPaths::GetBaseFilename(FPaths::GetPath(ProjectDir)); // "QAGame"
	return FText::Format(LOCTEXT("ProjectLayoutsHeader", "{0} Layouts"), FText::FromString(ProjectDirName));
}

void FPrivateLayoutsMenu::MakeXLayoutsMenuInternal(UToolMenu* InToolMenu, const ELayoutsMenu InLayoutsMenu)
{
	// Display (if load-only) engine/project layouts and (always)  user layouts
	const TArray<FLayoutsMenu::ELayoutsType> LayoutSectionTypes = InLayoutsMenu == ELayoutsMenu::Load
		? TArray<FLayoutsMenu::ELayoutsType>({FLayoutsMenu::ELayoutsType::Engine, FLayoutsMenu::ELayoutsType::Project, FLayoutsMenu::ELayoutsType::User})
		: /*InLayoutsMenu == ELayoutsMenu::Save|Remove ? */ TArray<FLayoutsMenu::ELayoutsType>({FLayoutsMenu::ELayoutsType::User});
	for (const FLayoutsMenu::ELayoutsType LayoutSectionType : LayoutSectionTypes)
	{
		FToolMenuSection& Section = (
			LayoutSectionType == FLayoutsMenu::ELayoutsType::Engine ? InToolMenu->AddSection("DefaultLayouts", LOCTEXT("DefaultLayoutsHeading", "Default Layouts")) :
			LayoutSectionType == FLayoutsMenu::ELayoutsType::Project ? InToolMenu->AddSection("ProjectDefaultLayouts", GetProjectLayoutSectionName()) :
			/*LayoutSectionType == FLayoutsMenu::ELayoutsType::User ?*/ InToolMenu->AddSection("UserDefaultLayouts", LOCTEXT("UserDefaultLayoutsHeading", "User Layouts")));
		// (Create if it does not exist and) Get LayoutsDirectory path
		const FString LayoutsDirectory = CreateAndGetDefaultLayoutDirInternal(LayoutSectionType);
		// Get Layout init files
		const TArray<FString> LayoutIniFileNames = GetIniFilesInFolderInternal(LayoutsDirectory);
		// If there are user Layout ini files, read and display them
		DisplayLayoutsInternal(Section, LayoutIniFileNames, LayoutsDirectory, InLayoutsMenu, LayoutSectionType);
	}
}

bool FPrivateLayoutsMenu::CanChooseLayoutWhenReadInternal()
{
	// All can be read
	return true;
}

bool FPrivateLayoutsMenu::CanChooseLayoutWhenWriteInternal(const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	// Only the layouts created by the user can be modified
	return (InLayoutsType == FLayoutsMenu::ELayoutsType::User);
}

void FPrivateLayoutsMenu::SaveLayoutWithoutRemovingTempLayoutFiles()
{
	// Save the layout into the Editor
	FGlobalTabmanager::Get()->SaveAllVisualState();
	// Write the saved layout to disk (if it has changed since the last time it was read/written)
	// We must set bRead = true. Otherwise, FLayoutsMenuLoad::ReloadCurrentLayout() would reload the old config file (because it would be cached on memory)
	const bool bRead = true;
	GConfig->Flush(bRead, GEditorLayoutIni);
}

bool FPrivateLayoutsMenu::CheckAskUserToClosePIESIE(const FText& InitialMessage)
{
	if (!GEditor)
	{
		ensureMsgf(false, TEXT("GEditor should not be false when CheckAskUserToClosePIESIE() is called."));
		return true;
	}
	// If PIE/SIE are not opened, return true
	const bool bIsPIEOrSIERunning = ((GEditor && GEditor->PlayWorld) || GIsPlayInEditorWorld);
	if (!bIsPIEOrSIERunning)
	{
		return true;
	}
	// If PIE/SIE are opened
	// FMessageDialog - Ask the user whether they wants to automatically close them and continue loading the layout
	const FText TextTitle = LOCTEXT("CheckAskUserToClosePIESIEIfYesHeaderPIE", "Close PIE/SIE?");
	const FText IfYesText = LOCTEXT("CheckAskUserToClosePIESIEIfYesBodyPIE", "If \"Yes\", your current game instances (PIE or SIE) will be closed. Any unsaved changes in those will also be lost.");
	const FText IfNoText = LOCTEXT("CheckAskUserToClosePIESIEIfNoBody", "If \"No\", you can manually reload the layout from the \"User Layouts\" section later.");
	const FText TextBody = FText::Format(LOCTEXT("ClosePIESIEAssetEditorsBody", "{0}\n\n{1}\n\n{2}"), InitialMessage, IfYesText, IfNoText);
	// Return if the user did not want to close them
	if (EAppReturnType::Yes != FMessageDialog::Open(EAppMsgType::YesNo, TextBody, TextTitle))
	{
		return false;
	}
	// Close PIE/SIE
	if (GEditor && GEditor->PlayWorld)
	{
		GEditor->EndPlayMap();
	}
	else
	{
		ensureMsgf(false,
			TEXT("This has not been tested because the code does not reach this by default. The layout is loaded through the Editor UI, and GIsPlayInEditorWorld should"
				" not have any kind of Editor UI, so it should not be possible to load a layout in that status."));
	}
	return true;
}

void FPrivateLayoutsMenu::SaveExportLayoutCommon(const FString& InDefaultDirectory, const bool bMustBeSavedInDefaultDirectory, const FText& InWhatIsThis,
	const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields)
{
	// Export/SaveAs the user-selected layout configuration files and load one of them
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		bool bWereFilesSelected = false;
		TArray<FString> LayoutFilePaths;
		bool bWasDialogOpened = bMustBeSavedInDefaultDirectory;
		// "Save Layout As..."
		if (bMustBeSavedInDefaultDirectory)
		{
			TArray<FName> LayoutNames;
			LayoutNames.Emplace(FLayoutSaveRestore::LoadSectionFromConfig(GEditorLayoutIni, "LayoutName").ToString());
			TArray<FName> LayoutDescriptions;
			LayoutDescriptions.Emplace(FLayoutSaveRestore::LoadSectionFromConfig(GEditorLayoutIni, "LayoutDescription").ToString());
			// We want to avoid the duplication of the file/name/description fields, so we add "Copy of " at the beginning of the name and description fields

			const FText NewLayoutName = LOCTEXT("NewLayoutName", "My New Layout");
			LayoutNames[0] = FName(NewLayoutName.ToString());

			const FText NewLayoutDescription = LOCTEXT("NewLayoutDescription", "Tooltip for My New Layout");
			LayoutDescriptions[0] = FName(NewLayoutDescription.ToString());

			// Create SWidget for saving the layout in its own SWindow and block the thread until it is finished
			const TSharedRef<FSaveLayoutDialogParams> SaveLayoutDialogParams = MakeShared<FSaveLayoutDialogParams>(InDefaultDirectory, TEXT(".ini"), LayoutNames, LayoutDescriptions);
			bWasDialogOpened = FSaveLayoutDialogUtils::CreateSaveLayoutAsDialogInStandaloneWindow(SaveLayoutDialogParams);
			bWereFilesSelected = SaveLayoutDialogParams->bWereFilesSelected;
			LayoutFilePaths = SaveLayoutDialogParams->LayoutFilePaths;
			LayoutNames = SaveLayoutDialogParams->LayoutNames;
			LayoutDescriptions = SaveLayoutDialogParams->LayoutDescriptions;

			// Update GEditorLayoutIni file if LayoutNames or LayoutDescriptions were modified by the user
			if (bWereFilesSelected && LayoutNames.Num() > 0 && LayoutDescriptions.Num() > 0 && (LayoutNames[0].ToString().Len() > 0 || LayoutDescriptions[0].ToString().Len() > 0))
			{
				checkf(LayoutNames.Num() == LayoutDescriptions.Num(), TEXT("There should be the same number of LayoutNames and LayoutDescriptions."));
				for (int32 Index = 0; Index < LayoutNames.Num(); ++Index)
				{
					const FString LayoutName = LayoutNames[Index].ToString();
					const FString LayoutDescription = LayoutDescriptions[Index].ToString();
					// Update fields
					FLayoutSaveRestore::SaveSectionToConfig(GEditorLayoutIni, "LayoutName", LayoutName);
					FLayoutSaveRestore::SaveSectionToConfig(GEditorLayoutIni, "LayoutDescription", LayoutDescription);
					// Flush file
					const bool bRead = true;
					GConfig->Flush(bRead, GEditorLayoutIni);
				}
			}
		}
		// "Export Layout..." (or "Save Layout As..." dialog could not be opened)
		if (!bWasDialogOpened)
		{
			// Open the "save file" dialog so the user can save their layout configuration file
			const FString DefaultFile = "";
			bWereFilesSelected = DesktopPlatform->SaveFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				TEXT("Export a Layout Configuration File"),
				InDefaultDirectory,
				DefaultFile,
				TEXT("Layout configuration files|*.ini|"),
				EFileDialogFlags::None, //EFileDialogFlags::Multiple, // Allow/Avoid multiple file selection
				LayoutFilePaths
			);
		}
		// If file(s) selected, copy them into the user layouts directory and load one of them
		if (bWereFilesSelected && LayoutFilePaths.Num() > 0)
		{
			// Iterate over selected layout ini files
			FString FirstGoodLayoutFile = "";
			for (const FString& LayoutFilePath : LayoutFilePaths)
			{
				// If writing in the right folder
				const FString LayoutFilePathAbsolute = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::GetPath(LayoutFilePath));
				const FString DefaultDirectoryAbsolute = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::GetPath(InDefaultDirectory));
				if (!bMustBeSavedInDefaultDirectory || (LayoutFilePathAbsolute == DefaultDirectoryAbsolute))
				{
					// Save in the user layout folder
					const FString& SourceFilePath = GEditorLayoutIni;
					const FString& TargetFilePath = LayoutFilePath;
					const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues = !bMustBeSavedInDefaultDirectory;
					const bool bShowSaveToast = true;
					TrySaveLayoutOrWarnInternal(SourceFilePath, TargetFilePath, InWhatIsThis, bCleanLayoutNameAndDescriptionFieldsIfNoSameValues,
						bShouldAskBeforeCleaningLayoutNameAndDescriptionFields, bShowSaveToast);
				}
				// If trying to write in a different folder (which is not allowed)
				else
				{
					// Warn the user that the file will not be copied in there
					FMessageDialog::Open(
						EAppMsgType::Ok,
						FText::Format(
							LOCTEXT("SaveAsFailedMsg", "In order to save the layout and allow Unreal to use it, you must save it in the predefined folder:\n{0}\n\nNevertheless, you tried to save it in:\n{1}\n\nIf you simply wish to export a copy of the current configuration in {1} (e.g., to later copy it into a different machine), you could use the \"Export Layout...\" functionality. However, Unreal would not be able to load it until you import it with \"Import Layout...\"."),
							FText::FromString(DefaultDirectoryAbsolute), FText::FromString(LayoutFilePathAbsolute)),
						LOCTEXT("SaveAsFailedMsg_Title", "Save As Failed"));
				}
			}
		}
	}
}

int32 FPrivateLayoutsMenu::GetNumberLayoutFiles(const FString& InLayoutsDirectory)
{
	// Get Layout init files in desired directory
	const TArray<FString> LayoutIniFileNames = GetIniFilesInFolderInternal(InLayoutsDirectory);
	// Count how many layout files exist
	int32 NumberLayoutFiles = 0;
	for (const FString& LayoutIniFileName : LayoutIniFileNames)
	{
		const FString LayoutFilePath = FPaths::Combine(InLayoutsDirectory, LayoutIniFileName);
		GConfig->UnloadFile(LayoutFilePath); // We must re-read it to avoid the Editor to use a previously cached name and description
		if (FLayoutSaveRestore::IsValidConfig(LayoutFilePath))
		{
			++NumberLayoutFiles;
		}
	}
	// Return result
	return NumberLayoutFiles;
}



/* FLayoutsMenuLoad public functions
 *****************************************************************************/

void FLayoutsMenuLoad::MakeLoadLayoutsMenu(UToolMenu* InToolMenu)
{
	// MakeLoadLayoutsMenu
	FPrivateLayoutsMenu::MakeXLayoutsMenuInternal(InToolMenu, FPrivateLayoutsMenu::ELayoutsMenu::Load);

	// Additional sections
	if (GetDefault<UEditorStyleSettings>()->bEnableUserEditorLayoutManagement)
	{
		FToolMenuSection& Section = InToolMenu->FindOrAddSection("UserDefaultLayouts");

		// Separator
		if (FLayoutsMenu::IsThereUserLayouts())
		{
			Section.AddSeparator("AdditionalSectionsSeparator");
		}

		// Import...
		{
			Section.AddMenuEntry(FMainFrameCommands::Get().ImportLayout);
		}
	}
}

bool FLayoutsMenuLoad::CanLoadChooseLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	return !FLayoutsMenu::IsLayoutChecked(InLayoutIndex, InLayoutsType) && FPrivateLayoutsMenu::CanChooseLayoutWhenReadInternal();
}

void FLayoutsMenuLoad::ReloadCurrentLayout()
{
	// If PIE, SIE, or any Asset Editors are opened, ask the user whether they want to automatically close them and continue loading the layout
	if (!FPrivateLayoutsMenu::CheckAskUserToClosePIESIE(LOCTEXT("AreYouSureToLoadHeader", "Are you sure you want to continue loading the selected layout profile?")))
	{
		return;
	}
	// Create duplicated ini file (OriginalEditorLayoutIniFilePath)
	// Explanation:
	//     Assume a layout is saved with (at least) a window that is dependent on a plugin. If that plugin is disabled and the editor restarted, that window will be saved on the layout but will
	//     not visually appear. We still wanna save the layout with it, so if its plugin is re-enabled, the window appear again. However, while the plugin is disabled, the layout ini file changes
	//     to reflect that the plugin is not opened.
	// Technical details:
	//     Rather than changing the string generated in the ini file (which could affect other parts of the code), we will duplicate the ini file when loaded. If the ini file is different than
	//     its duplicated copy, then some widget is missing (most probably due to disabled plugins). If that is the case, we will re-save the ini file without telling UE that it changed. This way,
	//     the ini file would match its original one, and it would only be re-modified if the user modifies the layout (but in that case it should no longer match the original one).
	const bool bShouldReplace = true;
	const bool bEvenIfReadOnly = true;
	const bool bCopyAttributes = false; // If true, it could e.g., copy the read-only flag of DefaultLayout.ini and make all the save/load stuff stop working
	const FString& OriginalEditorLayoutIniFilePath = FPrivateLayoutsMenu::GetOriginalEditorLayoutIniFilePathInternal();
	IFileManager::Get().Copy(*OriginalEditorLayoutIniFilePath, *GEditorLayoutIni, bShouldReplace, bEvenIfReadOnly, bCopyAttributes);
	GConfig->UnloadFile(OriginalEditorLayoutIniFilePath);
	// Disable config saving
	UAssetEditorSubsystem* AssetEditorSubsystem = (GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr);
	const bool bAreAssetEditorOpened = (AssetEditorSubsystem ? AssetEditorSubsystem->GetAllEditedAssets().Num() > 0 : false); // Are there asset editors opened?
	TOptional<bool> bAutoRestoreAndDisableSavingOverride;
	if (bAreAssetEditorOpened)
	{
		// Save open asset editors + disable manual saving (no need to close them with AssetEditorSubsystem->CloseAllAssetEditors)
		AssetEditorSubsystem->SaveOpenAssetEditors(true);
		bAutoRestoreAndDisableSavingOverride = AssetEditorSubsystem->GetAutoRestoreAndDisableSavingOverride();
		AssetEditorSubsystem->SetAutoRestoreAndDisableSavingOverride(true);
	}
	FGlobalTabmanager::Get()->SetCanSavePersistentLayouts(false);
	// Editor is reset on-the-fly
	EditorReinit();
	// Reenable config saving
	FGlobalTabmanager::Get()->SetCanSavePersistentLayouts(true);
	if (bAreAssetEditorOpened)
	{
		check(AssetEditorSubsystem);
		// Restore asset editors + disable manual saving and avoid trying to re-open the asset editors twice
		AssetEditorSubsystem->RestorePreviouslyOpenAssets();
		AssetEditorSubsystem->SetAutoRestoreAndDisableSavingOverride(bAutoRestoreAndDisableSavingOverride);
	}
	// Save layout and create duplicated ini file (DuplicatedEditorLayoutIniFilePath)
	FPrivateLayoutsMenu::SaveLayoutWithoutRemovingTempLayoutFiles();
	// If same file, remove temp files
	const bool bCheckTempFileToo = false;
	if (FPrivateLayoutsMenu::IsLayoutCheckedInternal(OriginalEditorLayoutIniFilePath, bCheckTempFileToo))
	{
		FPrivateLayoutsMenu::RemoveTempEditorLayoutIniFilesInternal();
	}
	// Else, create DuplicatedEditorLayoutIniFilePath
	else
	{
		const FString& DuplicatedEditorLayoutIniFilePath = FPrivateLayoutsMenu::GetDuplicatedEditorLayoutIniFilePathInternal();
		IFileManager::Get().Copy(*DuplicatedEditorLayoutIniFilePath, *GEditorLayoutIni, bShouldReplace, bEvenIfReadOnly, bCopyAttributes);
		GConfig->UnloadFile(DuplicatedEditorLayoutIniFilePath);
	}
}

void FLayoutsMenuLoad::LoadLayout(const FString& InLayoutPath, bool bInSkipLayoutReload)
{
	// Replace main layout with desired one
	const FString& SourceFilePath = InLayoutPath;
	const FString& TargetFilePath = GEditorLayoutIni;
	const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues = false;
	const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = false;
	const bool SucessfullySaved = FPrivateLayoutsMenu::TrySaveLayoutOrWarnInternal(SourceFilePath, TargetFilePath, LOCTEXT("LoadLayoutText", "loading the layout"),
		bCleanLayoutNameAndDescriptionFieldsIfNoSameValues, bShouldAskBeforeCleaningLayoutNameAndDescriptionFields);
	// Reload current layout
	if (SucessfullySaved && !bInSkipLayoutReload)
	{
		FLayoutsMenuLoad::ReloadCurrentLayout();
	}
}

void FLayoutsMenuLoad::LoadLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	// Replace main layout with desired one, reset layout & restart Editor
	LoadLayout(FLayoutsMenu::GetLayout(InLayoutIndex, InLayoutsType));
}

void FLayoutsMenuLoad::ImportLayout()
{
	// Import the user-selected layout configuration files and load one of them
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		// Open File Dialog so the user can select their desired layout configuration files
		TArray<FString> LayoutFilePaths;
		const FString LastDirectory = FPaths::ProjectContentDir();
		const FString DefaultDirectory = LastDirectory;
		const FString DefaultFile = "";
		const bool bWereFilesSelected = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			TEXT("Import a Layout Configuration File"),
			DefaultDirectory,
			DefaultFile,
			TEXT("Layout configuration files|*.ini|"),
			EFileDialogFlags::Multiple, //EFileDialogFlags::None, // Allow/Avoid multiple file selection
			LayoutFilePaths
		);
		// If file(s) selected, copy them into the user layouts directory and load one of them
		if (bWereFilesSelected && LayoutFilePaths.Num() > 0)
		{
			// (Create if it does not exist and) Get UserLayoutsDirectory path
			const FString UserLayoutsDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(FLayoutsMenu::ELayoutsType::User);
			// Iterate over selected layout ini files
			FString FirstGoodLayoutFile = "";
			const FText TrySaveLayoutOrWarnInternalText = LOCTEXT("ImportLayoutText", "importing the layout");
			for (const FString& LayoutFilePath : LayoutFilePaths)
			{
				// If file is a layout file, import it
				GConfig->UnloadFile(LayoutFilePath); // We must re-read it to avoid the Editor to use a previously cached name and description
				if (FLayoutSaveRestore::IsValidConfig(LayoutFilePath))
				{
					if (FirstGoodLayoutFile == "")
					{
						FirstGoodLayoutFile = LayoutFilePath;
					}
					// Save in the user layout folder
					const FString& SourceFilePath = LayoutFilePath;
					const FString& TargetFilePath = FPaths::Combine(FPaths::GetPath(UserLayoutsDirectory), FPaths::GetCleanFilename(LayoutFilePath));
					const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues = false;
					const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = false;
					FPrivateLayoutsMenu::TrySaveLayoutOrWarnInternal(SourceFilePath, TargetFilePath, TrySaveLayoutOrWarnInternalText,
						bCleanLayoutNameAndDescriptionFieldsIfNoSameValues, bShouldAskBeforeCleaningLayoutNameAndDescriptionFields);
				}
				// File is not a layout file, warn the user
				else
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("FileName"), FText::FromString(FPaths::ConvertRelativePathToFull(LayoutFilePath)));
					const FText TextBody = FText::Format(LOCTEXT("UnsuccessfulImportBody", "Unsuccessful import, {FileName} is not a layout configuration file!"), Arguments);
					const FText TextTitle = LOCTEXT("UnsuccessfulImportHeader", "Unsuccessful Import!");
					FMessageDialog::Open(EAppMsgType::Ok, TextBody, TextTitle);
				}
			}
			// If PIE, SIE, or any Asset Editors are opened, ask the user whether they wants to automatically close them and continue loading the layout
			if (!FPrivateLayoutsMenu::CheckAskUserToClosePIESIE(LOCTEXT("LayoutImportClosePIEAndEditorAssetsHeader", "The layout(s) were successfully imported into the \"User Layouts\" section. Do you want to continue loading the selected layout profile?")))
			{
				return;
			}
			// Replace current layout with first one
			if (FirstGoodLayoutFile != "")
			{
				const FString& SourceFilePath = FirstGoodLayoutFile;
				const FString& TargetFilePath = GEditorLayoutIni;
				const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues = false;
				const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = false;
				const bool SucessfullySaved = FPrivateLayoutsMenu::TrySaveLayoutOrWarnInternal(SourceFilePath, TargetFilePath, TrySaveLayoutOrWarnInternalText,
					bCleanLayoutNameAndDescriptionFieldsIfNoSameValues, bShouldAskBeforeCleaningLayoutNameAndDescriptionFields);
				// Reload current layout
				if (SucessfullySaved)
				{
					ReloadCurrentLayout();
				}
			}
		}
	}
}



/* FLayoutsMenuSave public functions
 *****************************************************************************/

void FLayoutsMenuSave::MakeSaveLayoutsMenu(UToolMenu* InToolMenu)
{
	if (GetDefault<UEditorStyleSettings>()->bEnableUserEditorLayoutManagement)
	{
		// MakeOverrideLayoutsMenu
		FPrivateLayoutsMenu::MakeXLayoutsMenuInternal(InToolMenu, FPrivateLayoutsMenu::ELayoutsMenu::Save);

		// Additional sections
		{
			FToolMenuSection& Section = InToolMenu->FindOrAddSection("UserDefaultLayouts");

			// Separator
			if (FLayoutsMenu::IsThereUserLayouts())
			{
				Section.AddSeparator("AdditionalSectionsSeparator");
			}

			// Save as...
			{
				Section.AddMenuEntry(FMainFrameCommands::Get().SaveLayoutAs);
			}

			// Export...
			{
				Section.AddMenuEntry(FMainFrameCommands::Get().ExportLayout);
			}
		}
	}
}

bool FLayoutsMenuSave::CanSaveChooseLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	return !FLayoutsMenu::IsLayoutChecked(InLayoutIndex, InLayoutsType) && FPrivateLayoutsMenu::CanChooseLayoutWhenWriteInternal(InLayoutsType);
}

void FLayoutsMenuSave::OverrideLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	// Default layouts should never be modified, so this function should never be called
	checkf(InLayoutsType == FLayoutsMenu::ELayoutsType::User, TEXT("Default layouts should never be modified, so this function should never be called."));

	// (Create if it does not exist and) Get LayoutsDirectory path
	const FString LayoutsDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(InLayoutsType);
	// Get Layout init files
	const TArray<FString> LayoutIniFileNames = FPrivateLayoutsMenu::GetIniFilesInFolderInternal(LayoutsDirectory);
	const FString DesiredLayoutFullPath = FPaths::Combine(FPaths::GetPath(LayoutsDirectory), LayoutIniFileNames[InLayoutIndex]);
	// Are you sure you want to do this?
	if (!FSaveLayoutDialogUtils::OverrideLayoutDialog(LayoutIniFileNames[InLayoutIndex]))
	{
		return;
	}

	// Loading the layout which is about to be overriden, so we can store its proper name.
	// The code which overrides the current layout ends up using the Display Name and Description of the currently loaded layout.
	// That's not always the desired one. Making sure of it by loading it.
	constexpr bool bSkipReload = true;
	FLayoutsMenuLoad::LoadLayout(DesiredLayoutFullPath, bSkipReload);

	// Target and source files
	const FString& SourceFilePath = GEditorLayoutIni;
	const FString& TargetFilePath = DesiredLayoutFullPath;
	// Update GEditorLayoutIni file
	SaveLayout();
	// Replace desired layout with current one
	const bool bCleanLayoutNameAndDescriptionFieldsIfNoSameValues = true;
	const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = false;
	const bool bShowSaveToast = true;
	FPrivateLayoutsMenu::TrySaveLayoutOrWarnInternal(SourceFilePath, TargetFilePath, LOCTEXT("OverrideLayoutText", "overriding the layout"),
		bCleanLayoutNameAndDescriptionFieldsIfNoSameValues, bShouldAskBeforeCleaningLayoutNameAndDescriptionFields, bShowSaveToast);
}

void FLayoutsMenuSave::SaveLayout()
{
	// Save layout
	FPrivateLayoutsMenu::SaveLayoutWithoutRemovingTempLayoutFiles();
	// Remove temporary Editor Layout ini files if the layout (thus also GEditorLayoutIni) changed
	const bool bCheckTempFileToo = false;
	const FString& DuplicatedEditorLayoutIniFilePath = FPrivateLayoutsMenu::GetDuplicatedEditorLayoutIniFilePathInternal();
	if (!FPrivateLayoutsMenu::IsLayoutCheckedInternal(DuplicatedEditorLayoutIniFilePath, bCheckTempFileToo))
	{
		FPrivateLayoutsMenu::RemoveTempEditorLayoutIniFilesInternal();
	}
}

void FLayoutsMenuSave::SaveLayoutAs()
{
	// Update GEditorLayoutIni file
	SaveLayout();
	// Copy GEditorLayoutIni into desired file
	const FString DefaultDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(FLayoutsMenu::ELayoutsType::User);
	const bool bMustBeSavedInDefaultDirectory = true;
	const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = false;
	FPrivateLayoutsMenu::SaveExportLayoutCommon(DefaultDirectory, bMustBeSavedInDefaultDirectory, LOCTEXT("SaveLayoutText", "saving the layout"),
		bShouldAskBeforeCleaningLayoutNameAndDescriptionFields);
}

void FLayoutsMenuSave::ExportLayout()
{
	// Update GEditorLayoutIni file
	SaveLayout();
	// Copy GEditorLayoutIni into desired file
	const FString DefaultDirectory = FPaths::ProjectContentDir();
	const bool bMustBeSavedInDefaultDirectory = false;
	const bool bShouldAskBeforeCleaningLayoutNameAndDescriptionFields = true;
	FPrivateLayoutsMenu::SaveExportLayoutCommon(DefaultDirectory, bMustBeSavedInDefaultDirectory, LOCTEXT("ExportLayoutText", "exporting the layout"),
		bShouldAskBeforeCleaningLayoutNameAndDescriptionFields);
}



/* FLayoutsMenuRemove public functions
 *****************************************************************************/

void FLayoutsMenuRemove::MakeRemoveLayoutsMenu(UToolMenu* InToolMenu)
{
	if (GetDefault<UEditorStyleSettings>()->bEnableUserEditorLayoutManagement)
	{
		// MakeRemoveLayoutsMenu
		FPrivateLayoutsMenu::MakeXLayoutsMenuInternal(InToolMenu, FPrivateLayoutsMenu::ELayoutsMenu::Remove);

		// Additional sections
		{
			FToolMenuSection& Section = InToolMenu->FindOrAddSection("UserDefaultLayouts");

			// Separator
			if (FLayoutsMenu::IsThereUserLayouts())
			{
				Section.AddSeparator("AdditionalSectionsSeparator");
			}

			// Remove all
			Section.AddMenuEntry(FMainFrameCommands::Get().RemoveUserLayouts);
		}
	}
}

bool FLayoutsMenuRemove::CanRemoveChooseLayout(const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	return FPrivateLayoutsMenu::CanChooseLayoutWhenWriteInternal(InLayoutsType);
}

static FString GetLayoutJsonFileName(const FString& InConfigFileName)
{
	const FString JsonFileName = FPaths::GetBaseFilename(InConfigFileName) + TEXT(".json");
#ifdef UE_SAVED_DIR_OVERRIDE
	const FString UserSettingsPath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), TEXT(UE_STRINGIZE(UE_SAVED_DIR_OVERRIDE)), TEXT("Editor"), JsonFileName);
#else
	const FString UserSettingsPath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), FApp::GetEpicProductIdentifier(), TEXT("Editor"), JsonFileName);
#endif
	return UserSettingsPath;
}

void FLayoutsMenuRemove::RemoveLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType)
{
	// Default layouts should never be modified, so this function should never be called
	checkf(InLayoutsType == FLayoutsMenu::ELayoutsType::User, TEXT("Default layouts should never be modified, so this function should never be called."));

	// (Create if it does not exist and) Get LayoutsDirectory path
	const FString LayoutsDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(InLayoutsType);
	// Get Layout init files
	const TArray<FString> LayoutIniFileNames = FPrivateLayoutsMenu::GetIniFilesInFolderInternal(LayoutsDirectory);
	const FString DesiredLayoutFullPath = FPaths::Combine(FPaths::GetPath(LayoutsDirectory), LayoutIniFileNames[InLayoutIndex]);
	// Are you sure you want to do this?
	const FText TextFileNameToRemove = FText::FromString(FPaths::GetBaseFilename(LayoutIniFileNames[InLayoutIndex]));
	const FText TextBody = FText::Format(LOCTEXT("ActionRemoveMsg", "Are you sure you want to permanently delete the layout profile \"{0}\"? This action cannot be undone."), TextFileNameToRemove);
	const FText TextTitle = FText::Format(LOCTEXT("RemoveUILayout_Title", "Remove UI Layout \"{0}\"?"), TextFileNameToRemove);
	if (EAppReturnType::Ok != FMessageDialog::Open(EAppMsgType::OkCancel, TextBody, TextTitle))
	{
		return;
	}
	// Remove layout
	FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*DesiredLayoutFullPath);

	// Remove JSON layout as well
	const FString TargetJsonFilepath = GetLayoutJsonFileName(LayoutIniFileNames[InLayoutIndex]);
	if (FLayoutSaveRestore::IsValidConfig(TargetJsonFilepath))
	{
		FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*TargetJsonFilepath);
	}
}

void FLayoutsMenuRemove::RemoveUserLayouts()
{
	// (Create if it does not exist and) Get UserLayoutsDirectory path
	const FString UserLayoutsDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(FLayoutsMenu::ELayoutsType::User);
	// Count how many layout files exist
	const int32 NumberUserLayoutFiles = FPrivateLayoutsMenu::GetNumberLayoutFiles(UserLayoutsDirectory);
	// If files to remove, warn user and remove them all
	if (NumberUserLayoutFiles > 0)
	{
		// Are you sure you want to do this?
		const FText TextBody = FText::Format(LOCTEXT("ActionRemoveAllUserLayoutMsg", "Are you sure you want to permanently remove {0} layout {0}|plural(one=profile,other=profiles)? This action cannot be undone."), NumberUserLayoutFiles);
		const FText TextTitle = LOCTEXT("RemoveAllUserLayouts_Title", "Remove All User-Created Layouts?");
		if (EAppReturnType::Ok != FMessageDialog::Open(EAppMsgType::OkCancel, TextBody, TextTitle))
		{
			return;
		}
		// Get User Layout init files
		const TArray<FString> UserLayoutIniFileNames = FPrivateLayoutsMenu::GetIniFilesInFolderInternal(UserLayoutsDirectory);
		// If there are user Layout ini files, read them
		for (const FString& UserLayoutIniFileName : UserLayoutIniFileNames)
		{
			// Remove file if it is a layout
			const FString LayoutFilePath = FPaths::Combine(UserLayoutsDirectory, UserLayoutIniFileName);
			GConfig->UnloadFile(LayoutFilePath); // We must re-read it to avoid the Editor to use a previously cached name and description
			if (FLayoutSaveRestore::IsValidConfig(LayoutFilePath))
			{
				FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*LayoutFilePath);
			}

			// Remove JSON layout as well
			const FString TargetJsonFilepath = GetLayoutJsonFileName(UserLayoutIniFileName);
			if (FLayoutSaveRestore::IsValidConfig(TargetJsonFilepath))
			{
				FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*TargetJsonFilepath);
			}
		}
	}
	// If no files to remove, warn user
	else
	{
		// Show reason
		const FText TextBody = LOCTEXT("UnsuccessfulRemoveLayoutBody", "There are no layout profile files created by the user, so none could be removed.");
		const FText TextTitle = LOCTEXT("UnsuccessfulRemoveLayoutHeader", "Unsuccessful Remove All User Layouts!");
		FMessageDialog::Open(EAppMsgType::Ok, TextBody, TextTitle);
	}
}



/* FLayoutsMenu public functions
 *****************************************************************************/

FString FLayoutsMenu::GetLayout(const int32 InLayoutIndex, const ELayoutsType InLayoutsType)
{
	// (Create if it does not exist and) Get LayoutsDirectory path, layout init files, and desired layout path
	const FString LayoutsDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(InLayoutsType);
	const TArray<FString> LayoutIniFileNames = FPrivateLayoutsMenu::GetIniFilesInFolderInternal(LayoutsDirectory);
	const FString DesiredLayoutFullPath = FPaths::Combine(FPaths::GetPath(LayoutsDirectory), LayoutIniFileNames[InLayoutIndex]);
	// Return full path
	return DesiredLayoutFullPath;
}

bool FLayoutsMenu::IsThereUserLayouts()
{
	// (Create if it does not exist and) Get UserLayoutsDirectory path
	const FString UserLayoutsDirectory = FPrivateLayoutsMenu::CreateAndGetDefaultLayoutDirInternal(ELayoutsType::User);
	// At least 1 user layout file?
	return FPrivateLayoutsMenu::GetNumberLayoutFiles(UserLayoutsDirectory) > 0;
}

bool FLayoutsMenu::IsLayoutChecked(const int32 InLayoutIndex, const ELayoutsType InLayoutsType)
{
#if PLATFORM_MAC // On Mac, each time a key is pressed, all menus are re-generated, stalling the Editor given that SaveLayout is slow on Mac because it does not caches as in Windows.
	return false;
#else
	// Check if the desired layout file matches the one currently loaded
	const bool bCheckTempFileToo = true;
	return FPrivateLayoutsMenu::IsLayoutCheckedInternal(GetLayout(InLayoutIndex, InLayoutsType), bCheckTempFileToo);
#endif
}

#undef LOCTEXT_NAMESPACE

#endif
