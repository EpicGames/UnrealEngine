// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"

class UToolMenu;

/**
 * Static helper functions for populating the "Layouts" menu.
 */
class FLayoutsMenu
{
public:
	enum class ELayoutsType
	{
		Engine,
		Project,
		User
	};

	/**
	 * Static
	 * Get the full (engine, project, or user) layout file path.
	 * Helper function for LoadLayoutI, SaveLayoutI, and RemoveLayoutI.
	 * @param InLayoutIndex Index associated with the desired layout profile ini file to be read/written.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 */
	static FString GetLayout(const int32 InLayoutIndex, const ELayoutsType InLayoutsType);

	/**
	 * Static
	 * Checks whether there are user-created layouts.
	 * @return true if there is at least a layout in the user layouts directory.
	 */
	static bool IsThereUserLayouts();

	/**
	 * Static
	 * Checks which (engine, project, or user) layout entry should be checked.
	 * @param InLayoutIndex Index from the selected layout.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 * @return true if the menu entry should be checked.
	 */
	static bool IsLayoutChecked(const int32 InLayoutIndex, const ELayoutsType InLayoutsType);
};

/**
 * Static load-related helper functions for populating the "Layouts" menu.
 */
class FLayoutsMenuLoad
{
public:
	/**
	 * Static
	 * It creates the layout load selection menu.
	 */
	static void MakeLoadLayoutsMenu(UToolMenu* InToolMenu);

	/**
	 * Static
	 * Checks if the load menu can choose the selected (engine, project, or user) layout to load it.
	 * @param InLayoutIndex Index from the selected layout.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 * @return true if the selected layout can be read.
	 */
	static bool CanLoadChooseLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * Static
	 * It re-loads the current Editor UI Layout (from GEditorLayoutIni).
	 * This function is used for many of the functions of FLayoutsMenuLoad.
	 */
	static void ReloadCurrentLayout();

	/**
	 * Static
	 * Load the visual layout state of the editor from an existing layout profile ini file, given its file path.
	 * @param InLayoutPath File path associated with the desired layout profile ini file to be read/written.
	 * @param bInSkipLayoutReload Prevents FLayoutsMenuLoad::ReloadCurrentLayout() from being called after loading the specified layout profile.
	 */
	static void LoadLayout(const FString& InLayoutPath, bool bInSkipLayoutReload = false);

	/**
	 * Static
	 * Load the visual layout state of the editor from an existing (engine, project, or user-based) layout profile ini file.
	 * @param InLayoutIndex Index associated with the desired layout profile ini file to be read/written.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 */
	static void LoadLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * Static
	 * Import a visual layout state of the editor from a custom directory path and with a custom file name chosen by the user.
	 * It saves it into the user layout folder, and loads it.
	 */
	static void ImportLayout();
};

/**
 * Static save-related helper functions for populating the "Layouts" menu.
 */
class FLayoutsMenuSave
{
public:
	/**
	 * Static
	 * It creates the layout save selection menu.
	 */
	static void MakeSaveLayoutsMenu(UToolMenu* InToolMenu);

	/**
	 * Static
	 * Checks if the save menu can choose the selected (engine, project, or user-based) layout to modify it.
	 * @param InLayoutIndex Index from the selected layout.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 * @return true if the selected layout can be modified/removed.
	 */
	static bool CanSaveChooseLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * Static
	 * Override the visual layout state of the editor in an existing (engine, project, or user-based) layout profile ini file.
	 * @param InLayoutIndex Index associated with the desired layout profile ini file to be read/written.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 */
	static void OverrideLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * Static
	 * Save the visual layout state of the editor (if changes to the layout have been made since the last time it was saved).
	 * If no changes has been made to the layout, the file is not updated (given that it would not be required).
	 * Any function that saves the layout (e.g., OverrideLayoutI, OverrideUserLayoutI, SaveLayoutAs, ExportLayout, etc.) should internally call this function.
	*/
	static void SaveLayout();

	/**
	 * Static
	 * Save the visual layout state of the editor with a custom file name chosen by the user.
	 */
	static void SaveLayoutAs();

	/**
	 * Static
	 * Export the visual layout state of the editor in a custom directory path and with a custom file name chosen by the user.
	 */
	static void ExportLayout();
};

/**
 * Static remove-related helper functions for populating the "Layouts" menu.
 */
class FLayoutsMenuRemove
{
public:
	/**
	 * Static
	 * It creates the layout remove selection menu.
	 */
	static void MakeRemoveLayoutsMenu(UToolMenu* InToolMenu);

	/**
	 * Static
	 * Checks if the remove menu can choose the selected (engine, project, or user-based) layout to remove it.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 * @return true if the selected layout can be modified/removed.
	 */
	static bool CanRemoveChooseLayout(const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * Static
	 * Remove an existing (engine, project, or user-based) layout profile ini file.
	 * @param InLayoutIndex Index associated with the desired layout profile ini file to be read/written.
	 * @param InLayoutsType ELayoutsType associated with the desired type of layout (engine, project, or user-based).
	 */
	static void RemoveLayout(const int32 InLayoutIndex, const FLayoutsMenu::ELayoutsType InLayoutsType);

	/**
	 * Static
	 * Remove all the layout customizations created by the user.
	 */
	static void RemoveUserLayouts();
};

#endif
