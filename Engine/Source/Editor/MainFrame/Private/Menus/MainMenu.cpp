// Copyright Epic Games, Inc. All Rights Reserved.

#include "Menus/MainMenu.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Interfaces/IMainFrameModule.h"
#include "DesktopPlatformModule.h"
#include "ISourceControlModule.h"
#include "SourceControlProviders.h"
#include "IUndoHistoryEditorModule.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "SourceCodeNavigation.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Styling/AppStyle.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/Transactor.h"
#include "Settings/EditorExperimentalSettings.h"
#include "UnrealEdGlobals.h"
#include "Frame/MainFrameActions.h"
#include "Menus/LayoutsMenu.h"
#include "Menus/RecentProjectsMenu.h"
#include "Menus/SettingsMenu.h"
#include "ToolMenus.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "Framework/Commands/GenericCommands.h"
#include "ITranslationEditor.h"
#include "LauncherPlatformModule.h"
#include "MainFrameLog.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "MainFileMenu"

namespace UE::MainMenu::Private
{

void MakeSpawnerToolMenuEntry(TSharedPtr<FTabManager> TabManager, FToolMenuSection& InSection, const TSharedPtr<FTabSpawnerEntry> &InSpawner)
{
	if (!InSpawner->IsHidden())
	{
		InSection.AddMenuEntry(
			NAME_None,
			InSpawner->GetDisplayName().IsEmpty() ? FText::FromName(InSpawner->GetTabType()) : InSpawner->GetDisplayName(),
			InSpawner->GetTooltipText(),
			InSpawner->GetIcon(),
			TabManager->GetUIActionForTabSpawnerMenuEntry(InSpawner),
			EUserInterfaceActionType::Check
		);
	}
}

void PopulateTabSpawnerToolMenu_Helper(UToolMenu* InMenu, TSharedPtr<FTabManager> TabManager, TSharedRef<FWorkspaceItem> InMenuStructure, TSharedRef< TArray< TWeakPtr<FTabSpawnerEntry> > > AllSpawners, const int32 RecursionLevel, bool PutLabelOnSection)
{
	FToolMenuSection& Section = InMenu->FindOrAddSection(InMenuStructure->GetFName());
	if (PutLabelOnSection)
	{
		Section.Label = InMenuStructure->GetDisplayName();
	}

	for (const TSharedRef<FWorkspaceItem>& Child : InMenuStructure->GetChildItems())
	{
		// Leaf nodes have valid spawner entries.
		const TSharedPtr<FTabSpawnerEntry> Spawner = Child->AsSpawnerEntry();
		if (Spawner.IsValid())
		{
			// Only show non-hidden items that have a valid spawner.
			if (AllSpawners->Contains(Spawner.ToSharedRef()))
			{
				MakeSpawnerToolMenuEntry(TabManager, Section, Spawner);
			}
		}
		else if (Child->HasChildrenIn(*AllSpawners))
		{
			// Reduce the depth of the menu structure. Create a section for every odd group and a submenu for every even.
			if (RecursionLevel % 2 == 0)
			{
				// Create a named section in the current menu and add all children there.
				PopulateTabSpawnerToolMenu_Helper(InMenu, TabManager, Child, AllSpawners, RecursionLevel+1, true);
			}
			else
			{
				// Create a submenu and add all children there.
				FToolMenuEntry& SubMenu = Section.AddSubMenu(
					Child->GetFName(),
					Child->GetDisplayName(),
					Child->GetTooltipText(),
					FNewToolMenuDelegate::CreateStatic(&PopulateTabSpawnerToolMenu_Helper, TabManager, Child, AllSpawners, RecursionLevel+1, false),
					FToolUIActionChoice(),
					EUserInterfaceActionType::Button,
					false,
					Child->GetIcon()
				);
			}
		}
	}
}

void PopulateTabSpawnerToolMenu(const TSharedPtr<FTabManager>& TabManager, UToolMenu* InMenu, TSharedRef<FWorkspaceItem> MenuStructure, bool bIncludeOrphanedMenus)
{
	TSharedRef< TArray< TWeakPtr<FTabSpawnerEntry> > > AllSpawners = MakeShared< TArray< TWeakPtr<FTabSpawnerEntry> > >(TabManager->CollectSpawners());

	if (bIncludeOrphanedMenus)
	{
		FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);

		// Put all orphaned spawners at the top of the menu so programmers go and find them a nice home.
		for (const TWeakPtr<FTabSpawnerEntry>& WeakSpawner : *AllSpawners)
		{
			const TSharedPtr<FTabSpawnerEntry> Spawner = WeakSpawner.Pin();
			if (!Spawner)
			{
				continue;
			}
			
			const bool bHasNoPlaceInMenuStructure = !Spawner->GetParent().IsValid();
			if ( bHasNoPlaceInMenuStructure )
			{
				MakeSpawnerToolMenuEntry(TabManager, UnnamedSection, Spawner);
			}
		}
	}
	
	PopulateTabSpawnerToolMenu_Helper(InMenu, TabManager, MenuStructure, AllSpawners, 0, false);
}

void PopulateTabSpawnerToolSection(TSharedPtr<FTabManager> TabManager, FToolMenuSection& InSection, const FName& TabType)
{
	TSharedPtr<FTabSpawnerEntry> Spawner = TabManager->FindTabSpawnerFor(TabType);
	if (Spawner.IsValid())
	{
		MakeSpawnerToolMenuEntry(TabManager, InSection, Spawner);
	}
	else
	{
		UE_LOGF(LogMainFrame, Warning, "PopulateTabSpawnerMenu failed to find entry for %ls", *(TabType.ToString()));
	}
}

} // namespace UE::MainMenu::Private

void FMainMenu::RegisterFileMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* FileMenu = ToolMenus->RegisterMenu("MainFrame.MainMenu.File");

	FileMenu->AddSection("FileOpen", LOCTEXT("FileOpenHeading", "Open"), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	FToolMenuSection& FileAssetSection = FileMenu->FindOrAddSection("FileAsset");;
	{
		FileAssetSection.InsertPosition = FToolMenuInsert("FileOpen", EToolMenuInsertType::After);
		// Open Asset...
		FileAssetSection.AddMenuEntry(FGlobalEditorCommonCommands::Get().SummonOpenAssetDialog);
	}


	FToolMenuSection& FileSaveSection = FileMenu->AddSection("FileSave", LOCTEXT("FileSaveHeading", "Save"), FToolMenuInsert("FileAsset", EToolMenuInsertType::After));
	{

		// Save All
		FileSaveSection.AddMenuEntry(FMainFrameCommands::Get().SaveAll);

		// Choose specific files to save
		FileSaveSection.AddMenuEntry(FMainFrameCommands::Get().ChooseFilesToSave);
	}

	RegisterFileProjectMenu();
	RegisterExitMenuItems();
}

#undef LOCTEXT_NAMESPACE


#define LOCTEXT_NAMESPACE "MainEditMenu"

void FMainMenu::RegisterEditMenu()
{
	UToolMenu* EditMenu = UToolMenus::Get()->RegisterMenu("MainFrame.MainMenu.Edit");
	{
		FToolMenuSection& Section = EditMenu->AddSection("EditHistory", LOCTEXT("HistoryHeading", "History"));
		struct Local
		{
			/** @return Returns a dynamic text string for Undo that contains the name of the action */
			static FText GetUndoLabelText()
			{
				return FText::Format(LOCTEXT("DynamicUndoLabel", "Undo {0}"), GUnrealEd->Trans->GetUndoContext().Title);
			}

			/** @return Returns a dynamic text string for Redo that contains the name of the action */
			static FText GetRedoLabelText()
			{
				return FText::Format(LOCTEXT("DynamicRedoLabel", "Redo {0}"), GUnrealEd->Trans->GetRedoContext().Title);
			}
		};

		// Undo
		TAttribute<FText> DynamicUndoLabel;
		DynamicUndoLabel.BindStatic(&Local::GetUndoLabelText);
		Section.AddMenuEntry(FGenericCommands::Get().Undo, DynamicUndoLabel); // TAttribute< FString >::Create( &Local::GetUndoLabelText ) );

		// Redo
		TAttribute< FText > DynamicRedoLabel;
		DynamicRedoLabel.BindStatic( &Local::GetRedoLabelText );
		Section.AddMenuEntry(FGenericCommands::Get().Redo, DynamicRedoLabel); // TAttribute< FString >::Create( &Local::GetRedoLabelText ) );

		// Show undo history
		Section.AddMenuEntry(
			"UndoHistory",
			LOCTEXT("UndoHistoryTabTitle", "Undo History"),
			LOCTEXT("UndoHistoryTooltipText", "View the entire undo history."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "UndoHistory.TabIcon"),
			FUIAction(FExecuteAction::CreateLambda([](){  IUndoHistoryEditorModule::Get().ExecuteOpenUndoHistory(); } ))
			);
		// IUndoHistoryEditorModule registers the nomad tab spawner. Ensure it's registered before SLevelEditor::RestoreContentArea restores the layout.
		IUndoHistoryEditorModule::Get();
	}

	{
		FToolMenuSection& Section = EditMenu->AddSection("Configuration", LOCTEXT("ConfigurationHeading", "Configuration"));
		if (GetDefault<UEditorStyleSettings>()->bExpandConfigurationMenus)
		{
			Section.AddSubMenu(
				"EditorPreferencesSubMenu",
				LOCTEXT("EditorPreferencesSubMenuLabel", "Editor Preferences"),
				LOCTEXT("EditorPreferencesSubMenuToolTip", "Configure the behavior and features of this Editor"),
				FNewToolMenuDelegate::CreateStatic(&FSettingsMenu::MakeMenu, FName("Editor")),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon")
			);

			Section.AddSubMenu(
				"ProjectSettingsSubMenu",
				LOCTEXT("ProjectSettingsSubMenuLabel", "Project Settings"),
				LOCTEXT("ProjectSettingsSubMenuToolTip", "Change the settings of the currently loaded project"),
				FNewToolMenuDelegate::CreateStatic(&FSettingsMenu::MakeMenu, FName("Project")),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon")
			);
		}
		else
		{
			const FGlobalEditorCommonCommands& GlobalEditorCommands = FGlobalEditorCommonCommands::Get();
			Section.AddMenuEntry(GlobalEditorCommands.OpenEditorPreferences);
			Section.AddMenuEntry(GlobalEditorCommands.OpenKeyboardShortcuts);
			Section.AddMenuEntry(GlobalEditorCommands.OpenProjectSettings);
		}

		Section.AddDynamicEntry("PluginsEditor", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			//@todo The tab system needs to be able to be extendable by plugins [9/3/2013 Justin.Sargent]
			if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				UE::MainMenu::Private::PopulateTabSpawnerToolSection(FGlobalTabmanager::Get().ToSharedPtr(), InSection, FName(TEXT("PluginsEditor")));
			}
		}));
	}
}

#undef LOCTEXT_NAMESPACE

#define LOCTEXT_NAMESPACE "MainWindowMenu"

void FMainMenu::RegisterWindowMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MainFrame.MainMenu.Window");

	// Level Editor, General, and Testing sections
	// Automatically populate tab spawners from TabManager
	Menu->AddDynamicSection("TabManagerSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		if (USlateTabManagerContext* TabManagerContext = InMenu->FindContext<USlateTabManagerContext>())
		{
			if (TSharedPtr<FTabManager> TabManager = TabManagerContext->TabManager.Pin())
			{
				// The global tab manager will be the tab manager for nomad tabs that appear docked as major tabs. However major tabs are not spawned
				// via the window menu so ignore anything from the global tab manager since it is responsible for major tabs only
				if(TabManager != FGlobalTabmanager::Get())
				{
					// Local editor tabs
					{
						constexpr bool bDisplayOrphanedTabs = false;
						UE::MainMenu::Private::PopulateTabSpawnerToolMenu(TabManager, InMenu, TabManager->GetLocalWorkspaceMenuRoot(), bDisplayOrphanedTabs);
					}

					// General tabs
					{
						constexpr bool bDisplayOrphanedTabs = true;
						const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
						UE::MainMenu::Private::PopulateTabSpawnerToolMenu(TabManager, InMenu, MenuStructure.GetStructureRoot(), bDisplayOrphanedTabs);
					}
				}
			}
		}
	}));

	// Get content section
	Menu->AddSection("GetContent", NSLOCTEXT("MainAppMenu", "GetContentHeader", "Get Content"));

	// Layout section
	{
		FToolMenuSection& Section = Menu->AddSection("WindowLayout", NSLOCTEXT("MainAppMenu", "LayoutManagementHeader", "Layout"));
		// Load Layout
		Section.AddEntry(FToolMenuEntry::InitSubMenu(
			"LoadLayout",
			NSLOCTEXT("LayoutMenu", "LayoutLoadHeader", "Load Layout"),
			NSLOCTEXT("LayoutMenu", "LoadLayoutsSubMenu_ToolTip", "Load a layout configuration from disk"),
			FNewToolMenuDelegate::CreateStatic(&FLayoutsMenuLoad::MakeLoadLayoutsMenu),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.LoadLayout") 
		));
		// Save and Remove Layout
		// Opposite to "Load Layout", Save and Remove are dynamic, i.e., they can be enabled/removed depending on the value of
		// the variable GetDefault<UEditorStyleSettings>()->bEnableUserEditorLayoutManagement
		Section.AddDynamicEntry("OverrideAndRemoveLayout", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (GetDefault<UEditorStyleSettings>()->bEnableUserEditorLayoutManagement)
			{
				// Save Layout
				InSection.AddEntry(FToolMenuEntry::InitSubMenu(
					"OverrideLayout",
					NSLOCTEXT("LayoutMenu", "OverrideLayoutsSubMenu", "Save Layout"),
					NSLOCTEXT("LayoutMenu", "OverrideLayoutsSubMenu_ToolTip", "Save your current layout configuration on disk"),
					FNewToolMenuDelegate::CreateStatic(&FLayoutsMenuSave::MakeSaveLayoutsMenu),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.SaveLayout") 
				));
				// Remove Layout
				InSection.AddEntry(FToolMenuEntry::InitSubMenu(
					"RemoveLayout",
					NSLOCTEXT("LayoutMenu", "RemoveLayoutsSubMenu", "Remove Layout"),
					NSLOCTEXT("LayoutMenu", "RemoveLayoutsSubMenu_ToolTip", "Remove a layout configuration from disk"),
					FNewToolMenuDelegate::CreateStatic(&FLayoutsMenuRemove::MakeRemoveLayoutsMenu),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.RemoveLayout")
				));
			}
		}));

		// Enable Fullscreen section
#if !PLATFORM_MAC && !PLATFORM_LINUX // On Mac/Linux windowed fullscreen mode in the editor is currently unavailable

		const FName FullscreenSeparatorName = "FullscreenSeparator";
		// Separator
		Section.AddSeparator(FullscreenSeparatorName);

		// Fullscreen
		FToolMenuEntry& FullscreenEntry = Section.AddMenuEntry(FMainFrameCommands::Get().ToggleFullscreen);
		FullscreenEntry.InsertPosition.Position = EToolMenuInsertType::After;
		FullscreenEntry.InsertPosition.Name = FullscreenSeparatorName;
#endif
	}
}

#undef LOCTEXT_NAMESPACE


#define LOCTEXT_NAMESPACE "MainHelpMenu"

void FMainMenu::RegisterHelpMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MainFrame.MainMenu.Help");

	FToolMenuSection& ReferenceSection = Menu->AddSection("Reference", NSLOCTEXT("MainHelpMenu", "ReferenceSection", "Reference"));
	{
		ReferenceSection.AddMenuEntry(FMainFrameCommands::Get().DocumentationHome);
		ReferenceSection.AddMenuEntry(FMainFrameCommands::Get().BrowseAPIReference);
		ReferenceSection.AddMenuEntry(FMainFrameCommands::Get().BrowseCVars);
	}

	FToolMenuSection& CommunitySection = Menu->AddSection("Community", NSLOCTEXT("MainHelpMenu", "CommunitySection", "Community"));
	{
		CommunitySection.AddMenuEntry(FMainFrameCommands::Get().VisitCommunityHome);
		CommunitySection.AddMenuEntry(FMainFrameCommands::Get().VisitOnlineLearning);
		CommunitySection.AddMenuEntry(FMainFrameCommands::Get().VisitForums);
		CommunitySection.AddMenuEntry(FMainFrameCommands::Get().VisitSearchForAnswersPage);
		CommunitySection.AddMenuEntry(FMainFrameCommands::Get().VisitCommunitySnippets);
	}

	FToolMenuSection& BugReportingSection = Menu->AddSection("Support", NSLOCTEXT("MainHelpMenu", "SupportSection", "Support"));
	{
		BugReportingSection.AddMenuEntry(FMainFrameCommands::Get().VisitSupportWebSite);
		BugReportingSection.AddMenuEntry(FMainFrameCommands::Get().ReportABug);
		BugReportingSection.AddMenuEntry(FMainFrameCommands::Get().OpenIssueTracker);
	}

	FToolMenuSection& HelpApplicationSection = Menu->AddSection("HelpApplication", NSLOCTEXT("MainHelpMenu", "Application", "Application"));
	{

#if !PLATFORM_MAC // Handled by app's menu in menu bar
		HelpApplicationSection.AddMenuEntry(FMainFrameCommands::Get().AboutUnrealEd);
#endif

		HelpApplicationSection.AddMenuEntry(FMainFrameCommands::Get().CreditsUnrealEd);
		HelpApplicationSection.AddSeparator("EpicGamesHelp");
		HelpApplicationSection.AddMenuEntry(FMainFrameCommands::Get().VisitEpicGamesDotCom);
	}
}

#undef LOCTEXT_NAMESPACE

TSharedRef<SWidget> FMainMenu::MakeMainMenu(const TSharedPtr<FTabManager>& TabManager, const FName MenuName, FToolMenuContext& ToolMenuContext)
{
	// Cache all project names once
	FMainFrameActionCallbacks::CacheProjectNames();

	RegisterMainMenu();

	RegisterNomadMainMenu();

	ToolMenuContext.AppendCommandList(FMainFrameCommands::ActionList);

	USlateTabManagerContext* ContextObject = NewObject<USlateTabManagerContext>();
	ContextObject->TabManager = TabManager;
	ToolMenuContext.AddObject(ContextObject);

	// Create the menu bar!
	TSharedRef<SWidget> MenuBarWidget = UToolMenus::Get()->GenerateWidget(MenuName, ToolMenuContext);
	if (MenuBarWidget != SNullWidget::NullWidget)
	{
		// Tell tab-manager about the multi-box for platforms with a global menu bar
		TSharedRef<SMultiBoxWidget> MultiBoxWidget = StaticCastSharedRef<SMultiBoxWidget>(MenuBarWidget);
		TabManager->SetMenuMultiBox(ConstCastSharedRef<FMultiBox>(MultiBoxWidget->GetMultiBox()), MultiBoxWidget);
	}

	return MenuBarWidget;
}

void FMainMenu::RegisterMainMenu()
{
#define LOCTEXT_NAMESPACE "MainMenu"

	static const FName MainMenuName("MainFrame.MainMenu");
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered(MainMenuName))
	{
		return;
	}

	RegisterFileMenu();
	RegisterEditMenu();
	RegisterWindowMenu();
	RegisterToolsMenu();
	RegisterHelpMenu();

	UToolMenu* MenuBar = ToolMenus->RegisterMenu(MainMenuName, NAME_None, EMultiBoxType::MenuBar);

	static const FName MainMenuStyleName("WindowMenuBar");

	MenuBar->StyleName = MainMenuStyleName;

	MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"File",
		LOCTEXT("FileMenu", "File"),
		LOCTEXT("FileMenu_ToolTip", "Open the file menu")
	);

	MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"Edit",
		LOCTEXT("EditMenu", "Edit"),
		LOCTEXT("EditMenu_ToolTip", "Open the edit menu")
		);

	MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"Window",
		LOCTEXT("WindowMenu", "Window"),
		LOCTEXT("WindowMenu_ToolTip", "Open new windows or tabs.")
		);

	MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"Tools",
		LOCTEXT("ToolsMenu", "Tools"),
		LOCTEXT("ToolsMenu_ToolTip", "Level Tools")
		);


	MenuBar->AddSubMenu(
		"MainMenu",
		NAME_None,
		"Help",
		LOCTEXT("HelpMenu", "Help"),
		LOCTEXT("HelpMenu_ToolTip", "Open the help menu")
	);
}

#undef LOCTEXT_NAMESPACE


#define LOCTEXT_NAMESPACE "MainTabMenu"

void FMainMenu::RegisterFileProjectMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* MainTabFileMenu = ToolMenus->ExtendMenu("MainFrame.MainTabMenu.File");
	FToolMenuSection& Section = MainTabFileMenu->AddSection("FileProject", LOCTEXT("ProjectHeading", "Project"));

	if (GetDefault<UEditorStyleSettings>()->bShowProjectMenus)
	{
		Section.AddMenuEntry(FMainFrameCommands::Get().NewProject);
		Section.AddMenuEntry(FMainFrameCommands::Get().OpenProject);
		/*
		MenuBuilder.AddMenuEntry( FMainFrameCommands::Get().LocalizeProject,
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("LocalizeProjectToolTip", "Gather text from your project and import/export translations.")
			);
			*/
			/*
			MenuBuilder.AddSubMenu(
				LOCTEXT("CookProjectSubMenuLabel", "Cook Project"),
				LOCTEXT("CookProjectSubMenuToolTip", "Cook your project content for debugging"),
				FNewMenuDelegate::CreateStatic( &FCookContentMenu::MakeMenu ), false, FSlateIcon()
			);
			*/


		Section.AddMenuEntry(FMainFrameCommands::Get().ZipUpProject);
		Section.AddMenuEntry(FMainFrameCommands::Get().OpenCurrentProjectDirectory);

		if (FMainFrameActionCallbacks::RecentProjects.Num() > 0)
		{
			Section.AddSubMenu(
				"RecentProjects",
				LOCTEXT("SwitchProjectSubMenu", "Recent Projects"),
				LOCTEXT("SwitchProjectSubMenu_ToolTip", "Select a project to switch to"),
				FNewToolMenuDelegate::CreateStatic(&FRecentProjectsMenu::MakeMenu),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.RecentProjects")
			);
		}
	}
}

void FMainMenu::RegisterToolsMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("MainFrame.MainMenu.Tools");
	FToolMenuSection& Section = Menu->AddSection("Programming", LOCTEXT("ProgrammingHeading", "Programming"));


	FText ShortIDEName = FSourceCodeNavigation::GetSelectedSourceCodeIDE();
	FSlateIcon OpenIDEIcon = FSourceCodeNavigation::GetOpenSourceCodeIDEIcon();
	FSlateIcon RefreshIDEIcon = FSourceCodeNavigation::GetRefreshSourceCodeIDEIcon(); 

	Section.AddMenuEntry( FMainFrameCommands::Get().AddCodeToProject,
		TAttribute<FText>(),
		FText::Format(LOCTEXT("AddCodeToProjectTooltip", "Adds C++ code to the project. The code can only be compiled if you have {0} installed."), ShortIDEName)
	);

	Section.AddDynamicEntry("CodeProject", FNewToolMenuSectionDelegate::CreateLambda([ShortIDEName, RefreshIDEIcon, OpenIDEIcon](FToolMenuSection& InSection)
	{
		if (FSourceCodeNavigation::DoesModuleSolutionExist())
		{
			InSection.AddMenuEntry( FMainFrameCommands::Get().RefreshCodeProject,
				FText::Format(LOCTEXT("RefreshCodeProjectLabel", "Refresh {0} Project"), ShortIDEName),
				FText::Format(LOCTEXT("RefreshCodeProjectTooltip", "Refreshes your C++ code project in {0}."), ShortIDEName),
				RefreshIDEIcon	
			);
		}
		else
		{
			InSection.AddMenuEntry( FMainFrameCommands::Get().RefreshCodeProject,
				FText::Format(LOCTEXT("GenerateCodeProjectLabel", "Generate {0} Project"), ShortIDEName),
				FText::Format(LOCTEXT("GenerateCodeProjectTooltip", "Generates your C++ code project in {0}."), ShortIDEName),
				OpenIDEIcon
			);
		}
	}));

	Section.AddMenuEntry( FMainFrameCommands::Get().OpenIDE,
		FText::Format(LOCTEXT("OpenIDELabel", "Open {0}"), ShortIDEName),
		FText::Format(LOCTEXT("OpenIDETooltip", "Opens your C++ code in {0}."), ShortIDEName),
		OpenIDEIcon
	);


	// Level Editor, General, and Testing sections
	// Automatically populate tab spawners from TabManager
	Menu->AddDynamicSection("TabManagerSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		if (USlateTabManagerContext* TabManagerContext = InMenu->FindContext<USlateTabManagerContext>())
		{
			TSharedPtr<FTabManager> TabManager = TabManagerContext->TabManager.Pin();
			if (TabManager.IsValid())
			{
				// General tabs
				const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
				UE::MainMenu::Private::PopulateTabSpawnerToolMenu(TabManager, InMenu, MenuStructure.GetToolsStructureRoot(), true);
			}
		}
	}));

	// Experimental section
	{
		// This is a temporary home for the spawners of experimental features that must be explicitly enabled.
		// When the feature becomes permanent and need not check a flag, register a nomad spawner for it in the proper WorkspaceMenu category
		//const bool bLocalizationDashboard = GetDefault<UEditorExperimentalSettings>()->bEnableLocalizationDashboard;
		const bool bTranslationPicker = GetDefault<UEditorExperimentalSettings>()->bEnableTranslationPicker;

		// Make sure at least one is enabled before creating the section

		FToolMenuSection& ExperimentalSection = Menu->AddSection("ExperimentalTabSpawners", LOCTEXT("ExperimentalTabSpawnersHeading", "Experimental"));
		{
			// Translation Picker
			if (bTranslationPicker)
			{
				ExperimentalSection.AddMenuEntry(
					"TranslationPicker",
					LOCTEXT("TranslationPickerMenuItem", "Translation Picker"),
					LOCTEXT("TranslationPickerMenuItemToolTip", "Launch the Translation Picker to Modify Editor Translations"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationPicker.MenuIcon"),
					FUIAction(FExecuteAction::CreateLambda(
						[]()
						{
							FModuleManager::Get().LoadModuleChecked("TranslationEditor");
							ITranslationEditor::OpenTranslationPicker();
						}))
				);
			}
		}
	}

	// NOTE: This menu should be in sync with SourceControlMenuHelpers's SourceControl menu.

	FToolMenuSection& SourceControlSection = Menu->AddSection("Source Control", LOCTEXT("SourceControlHeading", "Revision Control"));

	SourceControlSection.AddDynamicEntry("SubmitTab", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

		// Access to the legacy 'Submit Files' dialog. Git / Subversion require this.
		if (SourceControlModule.IsEnabled() && !SourceControlModule.GetProvider().UsesChangelists() &&
			SourceControlModule.GetProvider().GetName() != SourceControlProviders::GetUrcProviderName())
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().SubmitContent,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Submit")
			);
		}
	}));

	SourceControlSection.AddDynamicEntry("ChangelistsTab", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

		if (SourceControlModule.GetProvider().UsesChangelists() || SourceControlModule.GetProvider().UsesUncontrolledChangelists())
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().ViewChangelists,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.ChangelistsTab")
			);
		}
	}));

	SourceControlSection.AddDynamicEntry("SnapshotHistoryTab", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

		if (SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName())
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().ViewSnapshotHistory,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.SnapshotHistoryTab")
			);
		}
	}));

	SourceControlSection.AddDynamicEntry("BranchExplorerTab", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

		if (SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName())
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().ViewBranchExplorer,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.BranchExplorerTab")
			);
		}
	}));

	SourceControlSection.AddDynamicEntry("CheckOutModifiedFilesTab", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

		if (SourceControlModule.GetProvider().UsesCheckout())
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().CheckOutModifiedFiles,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.Actions.CheckOut")
			);
		}
	}));

	SourceControlSection.AddDynamicEntry("ConnectToSourceControl", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

		if (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable())
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().ChangeSourceControlSettings,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.Actions.ChangeSettings")
			);
		}
		else
		{
			InSection.AddMenuEntry(
				FMainFrameCommands::Get().ConnectToSourceControl,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.Actions.Connect")
			);
		}
	}));
}

void FMainMenu::RegisterExitMenuItems()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	UToolMenu* MainTabFileMenu = ToolMenus->RegisterMenu("MainFrame.MainTabMenu.File", "MainFrame.MainMenu.File");

	

#if !PLATFORM_MAC // Handled by app's menu in menu bar
	{
		FToolMenuSection& Section = MainTabFileMenu->AddSection("Exit", LOCTEXT("Exit", "Exit"), FToolMenuInsert("FileProject", EToolMenuInsertType::After));
		Section.AddSeparator("Exit");
		Section.AddMenuEntry( FMainFrameCommands::Get().Exit );
	}
#endif
}

void FMainMenu::RegisterNomadMainMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName NomadMainMenuName("MainFrame.NomadMainMenu");
	if (!ToolMenus->IsMenuRegistered(NomadMainMenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(NomadMainMenuName, "MainFrame.MainMenu");
	}
}


#undef LOCTEXT_NAMESPACE
