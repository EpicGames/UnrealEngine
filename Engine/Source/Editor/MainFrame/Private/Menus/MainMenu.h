// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Framework/Docking/TabManager.h"

class FMenuBuilder;
struct FToolMenuContext;

/**
 * Unreal editor main frame Slate widget
 */
class FMainMenu
{
public:

	/**
	 * Static: Creates a widget for the main menu bar.
	 *
	 * @param TabManager Create the workspace menu based on this tab manager.
	 * @param MenuName Identifier associated with the menu.
	 * @return ToolMenuContext Context containing state.
	 */
	static TSharedRef<SWidget> MakeMainMenu( const TSharedPtr<FTabManager>& TabManager, const FName MenuName, FToolMenuContext& ToolMenuContext );

	/**
	 * Static: Registers main menu with menu system.
	 */
	static void RegisterMainMenu();

private:

	/**
	 * Called to fill the file menu's content.
	 *
	 */
	static void RegisterFileMenu();

	/**
	 * Called to fill the edit menu's content.
	 */
	static void RegisterEditMenu();

	/**
	 * Called to fill the app menu's content.
	 */
	static void RegisterWindowMenu();

	/**
	 * Called to fill the app menu's content.
	 */
	static void RegisterToolsMenu();

	/**
	 * Called to fill the help menu's content.
	 */
	static void RegisterHelpMenu();

	/**
	 * Called to fill the file menu's content.
	 */
	static void RegisterFileProjectMenu();

	/**
	 * Called to fill the file menu's recent and exit content.
	 */
	static void RegisterExitMenuItems();

	/**
	 * Called to fill the main menu for nomad tabs (these tabs generally have a less menu options)
	 */
	static void RegisterNomadMainMenu();

	/** 
	* Opens the legacy project launcher tab.
	*/
	UE_DEPRECATED(5.6, "invoke the tab directly")
	static void OpenProjectLauncher()
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FName(TEXT("LegacyProjectLauncher")));
	}

	/**
	* Opens the experimental Visual Logger tab.
	* Remove this when it is no longer experimental.
	*/
	static void OpenVisualLogger()
	{
		FModuleManager::Get().LoadModuleChecked<IModuleInterface>("LogVisualizer");
		FGlobalTabmanager::Get()->TryInvokeTab(FName(TEXT("VisualLogger")));
	}
};
