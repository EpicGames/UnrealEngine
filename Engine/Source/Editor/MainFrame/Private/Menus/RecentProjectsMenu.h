// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "ToolMenus.h"
#include "Frame/MainFrameActions.h"
#include "HAL/FileManager.h"
#include "Settings/EditorSettings.h"

#define LOCTEXT_NAMESPACE "FRecentProjectsMenu"

/**
 * Static helper class for populating the "Recent Projects" menu.
 */
class FRecentProjectsMenu
{
public:

	/**
	 * Creates the menu.
	 *
	 * @param Menu	The menu being populated.
	 */
	static void MakeMenu( UToolMenu* Menu )
	{
		for ( int32 ProjectIndex = 0; ProjectIndex < FMainFrameActionCallbacks::RecentProjects.Num() && ProjectIndex < FMainFrameCommands::Get().SwitchProjectCommands.Num(); ++ProjectIndex )
		{
			// If it is a project file, display the filename without extension. Otherwise just display the project name.
			const FString& ProjectName = FMainFrameActionCallbacks::RecentProjects[ ProjectIndex ].ProjectName;

			if (( IFileManager::Get().FileSize(*ProjectName) <= 0 ) ||
				( FPaths::GetProjectFilePath() == ProjectName ))
			{
				// Don't display project files that do not exist.
				continue;
			}

			FToolMenuSection& Section = Menu->FindOrAddSection("Recent");
			const FText DisplayName = FText::FromString( FPaths::GetBaseFilename(*ProjectName) );
			const FText Tooltip = FText::FromString( IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProjectName) );
			Section.AddMenuEntry( FMainFrameCommands::Get().SwitchProjectCommands[ ProjectIndex ], DisplayName, Tooltip ).Name = NAME_None;
		}
	}
};


#undef LOCTEXT_NAMESPACE
