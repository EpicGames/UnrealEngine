// Copyright Epic Games, Inc. All Rights Reserved.

#include "Menus/SaveLayoutDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Input/Reply.h"
#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Char.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SPrimaryButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SaveLayoutDialog"

/**
 * Note: Code for SSaveLayoutDialog obtained and modified from SAssetDialog
 *
 * A SWidget that generates a custom SCompoundWidget that allows the user to select the final file name where a layout ini file will be saved,
 * and optionally the displayed layout name and description.
 *
 * This SWidget is never meant to used by itself. Instead, FSaveLayoutDialogUtils::CreateSaveLayoutAsDialogInStandaloneWindow() should be called,
 * which additionally sets the right SWindow for SSaveLayoutDialog and blocks the thread until the user finishes interacting with it.
 */
class SSaveLayoutDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSaveLayoutDialog) {}

	SLATE_END_ARGS()

	SSaveLayoutDialog();

	virtual ~SSaveLayoutDialog();

	virtual void Construct(const FArguments& InArgs, const TSharedRef<FSaveLayoutDialogParams>& InSaveLayoutDialogParams);

private:
	FText GetFileNameText() const;

	/** Gets the visibility of the name error label */
	EVisibility GetNameErrorLabelVisibility() const;

	/** Gets the text to display in the name error label */
	FText GetNameErrorLabelText() const;

	/** Fired when the layout name box text is committed */
	void OnLayoutNameTextCommited(const FText& InText, ETextCommit::Type InCommitType);

	/** Fired when the layout description box text is committed */
	void OnLayoutDescriptionTextCommited(const FText& InText, ETextCommit::Type InCommitType);

	/** Determines if the Save button is enabled. */
	bool IsConfirmButtonEnabled() const;

	/** Handler for when the Save button is clicked */
	FReply OnConfirmClicked();

	/** Handler for when the cancel button is clicked */
	FReply OnCancelClicked();

	/** Used to commit the object path used for saving in this dialog */
	void CommitObjectPathForSave();

	/** Closes this dialog */
	void CloseDialog();

	bool IsValidFilePathForCreation(const FString& FilePath, const FString& FileName, FText& OutErrorMessage, bool bAllowExistingAsset = true) const;

	void UpdateInputValidity();

	TSharedPtr<FSaveLayoutDialogParams> SaveLayoutDialogParams;

	FString DefaultDirectory;

	FString FinalFileName;

	FString FinalFilePath;

	FName CurrentlyEnteredLayoutName;

	FName CurrentlyEnteredLayoutDescription;

	/** The error text from the last validity check */
	FText LastInputValidityErrorText;

	/** True if the last validity check returned that the class name/path is valid for creation */
	bool bLastInputValidityCheckSuccessful;
};



FSaveLayoutDialogParams::FSaveLayoutDialogParams(const FString& InDefaultDirectory, const FString& InFileExtension, const TArray<FName>& InLayoutNames, const TArray<FName>& InLayoutDescriptions)
	: DefaultDirectory(InDefaultDirectory)
	, FileExtension(InFileExtension)
	, LayoutNames(InLayoutNames)
	, LayoutDescriptions(InLayoutDescriptions)
{
}

void FSaveLayoutDialogUtils::SanitizeText(FString& InOutString)
{
	// Replace any special character by `_`
	for (TCHAR& InOutChar : InOutString)
	{
		if (!FChar::IsAlnum(InOutChar))
		{
			InOutChar = '_';
		}
	}
}

bool FSaveLayoutDialogUtils::OverrideLayoutDialog(const FString& LayoutIniFileName)
{
	// Are you sure you want to do this?
	const FText TextFileNameToRemove = FText::FromString(FPaths::GetBaseFilename(LayoutIniFileName));
	const FText TextBody = FText::Format(
		LOCTEXT("ActionOverrideLayoutMsg", "Are you sure you want to permanently override the layout profile \"{0}\" with the current layout profile? This action cannot be undone."), TextFileNameToRemove);
	const FText TextTitle = FText::Format(LOCTEXT("OverrideUILayout_Title", "Override UI Layout \"{0}\"?"), TextFileNameToRemove);
	return (EAppReturnType::Ok == FMessageDialog::Open(EAppMsgType::OkCancel, TextBody, TextTitle));
}

bool FSaveLayoutDialogUtils::CreateSaveLayoutAsDialogInStandaloneWindow(const TSharedRef<FSaveLayoutDialogParams>& InSaveLayoutDialogParams)
{
	// Create SSaveLayoutDialog
	TSharedRef<SSaveLayoutDialog> SaveLayoutDialog = SNew(SSaveLayoutDialog, InSaveLayoutDialogParams);

	// Create SWindow that contains SSaveLayoutDialog
	TSharedRef<SWindow> DialogWindow =
		SNew(SWindow)
		.Title(LOCTEXT("GenericAssetDialogWindowHeader", "Save Layout As"))
		.SizingRule(ESizingRule::Autosized);

	DialogWindow->SetContent(SaveLayoutDialog);

	// Launch SSaveLayoutDialog and block thread until user finishes with it
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
	if (MainFrameParentWindow.IsValid())
	{
		// Opposite to AddWindowAsNativeChild(), AddModalWindow() does not return until the modal window is closed
		FSlateApplication::Get().AddModalWindow(DialogWindow, MainFrameParentWindow.ToSharedRef());
		return true;
	}
	else
	{
		ensureMsgf(false, TEXT("Could not create \"Save Layout As...\" dialog, this should never happen."));
		return false;
	}
}

SSaveLayoutDialog::SSaveLayoutDialog()
	: bLastInputValidityCheckSuccessful(false)
{
}

SSaveLayoutDialog::~SSaveLayoutDialog()
{
}

void SSaveLayoutDialog::Construct(const FArguments& InArgs, const TSharedRef<FSaveLayoutDialogParams>& InSaveLayoutDialogParams)
{
	SaveLayoutDialogParams = InSaveLayoutDialogParams;

	// Set Default Saving Directory
	DefaultDirectory = SaveLayoutDialogParams->DefaultDirectory;

	// Fill initial (default) values
	if (SaveLayoutDialogParams->LayoutNames.Num() > 0)
	{
		ensureMsgf(SaveLayoutDialogParams->LayoutNames.Num() == 1, TEXT("This code is only ready for SaveLayoutDialogParams->LayoutNames.Num() == 1."));
		CurrentlyEnteredLayoutName = SaveLayoutDialogParams->LayoutNames[0];
		UpdateInputValidity();
	}
	if (SaveLayoutDialogParams->LayoutDescriptions.Num() > 0)
	{
		ensureMsgf(SaveLayoutDialogParams->LayoutDescriptions.Num() == 1, TEXT("This code is only ready for LayoutNames.Num() == 1."));
		CurrentlyEnteredLayoutDescription = SaveLayoutDialogParams->LayoutDescriptions[0];
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.0f)
		[
			SNew(SBox)
			.WidthOverride(600.f)
			[
				// Layout Name
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SGridPanel)
					.FillColumn(1, 1.0f)
					+SGridPanel::Slot(0,0)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("LayoutNameBoxLabel", "Name"))
						.Margin(FMargin(0, 0, 8, 8))
					]
					+SGridPanel::Slot(0,1)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("LayoutDescriptionBoxLabel", "Description"))
						.Margin(FMargin(0, 0, 8, 8))
					]
					+SGridPanel::Slot(1,0)
					.Padding(0, 0, 0, 8)
					[
						SNew(SEditableTextBox)
						.Text(FText::FromString(CurrentlyEnteredLayoutName.ToString()))
						.OnTextCommitted(this, &SSaveLayoutDialog::OnLayoutNameTextCommited)
						.OnTextChanged(this, &SSaveLayoutDialog::OnLayoutNameTextCommited, ETextCommit::Default)
						.SelectAllTextWhenFocused(true)
					]
					+SGridPanel::Slot(1,1)
					.Padding(0, 0, 0, 8)
					[
						SNew(SEditableTextBox)
						.Text(FText::FromString(CurrentlyEnteredLayoutDescription.ToString()))
						.HintText(LOCTEXT("LayoutDescriptionInputBoxHintText", "Optional"))
						.OnTextCommitted(this, &SSaveLayoutDialog::OnLayoutDescriptionTextCommited)
						.OnTextChanged(this, &SSaveLayoutDialog::OnLayoutDescriptionTextCommited, ETextCommit::Default)
						.SelectAllTextWhenFocused(true)
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					// Constant height, whether the label is visible or not
					SNew(SBox)
					.HeightOverride(20.f)
					[
						SNew(SBorder)
						.Visibility( this, &SSaveLayoutDialog::GetNameErrorLabelVisibility )
						.BorderImage( FAppStyle::GetBrush("AssetDialog.ErrorLabelBorder") )
						.Content()
						[
							SNew(STextBlock)
							.Text( this, &SSaveLayoutDialog::GetNameErrorLabelText )
							.ToolTipText(this, &SSaveLayoutDialog::GetNameErrorLabelText)
						]
					]
				]
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(0,0,8,0)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("SaveLayoutDialogSaveButton", "Save"))
						.IsEnabled(this, &SSaveLayoutDialog::IsConfirmButtonEnabled)
						.OnClicked(this, &SSaveLayoutDialog::OnConfirmClicked)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					[
						SNew(SButton)
						.Text(LOCTEXT("SaveLayoutDialogCancelButton", "Cancel"))
						.OnClicked(this, &SSaveLayoutDialog::OnCancelClicked)
					]
				]
			]
		]
	];
}

FText SSaveLayoutDialog::GetFileNameText() const
{
	return FText::FromString(FinalFileName);
}

EVisibility SSaveLayoutDialog::GetNameErrorLabelVisibility() const
{
	return GetNameErrorLabelText().IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;
}

FText SSaveLayoutDialog::GetNameErrorLabelText() const
{
	if (!bLastInputValidityCheckSuccessful)
	{
		return LastInputValidityErrorText;
	}

	return FText::GetEmpty();
}

void SSaveLayoutDialog::OnLayoutNameTextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	CurrentlyEnteredLayoutName = FName(InText.ToString());
	UpdateInputValidity();

	if (InCommitType == ETextCommit::OnEnter)
	{
		CommitObjectPathForSave();
	}
}

void SSaveLayoutDialog::OnLayoutDescriptionTextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	CurrentlyEnteredLayoutDescription = FName(InText.ToString());

	if (InCommitType == ETextCommit::OnEnter)
	{
		CommitObjectPathForSave();
	}
}

bool SSaveLayoutDialog::IsConfirmButtonEnabled() const
{
	return bLastInputValidityCheckSuccessful;
}

FReply SSaveLayoutDialog::OnConfirmClicked()
{
	CommitObjectPathForSave();
	return FReply::Handled();
}

FReply SSaveLayoutDialog::OnCancelClicked()
{
	bLastInputValidityCheckSuccessful = false;
	if (SaveLayoutDialogParams.IsValid())
	{
		SaveLayoutDialogParams->bWereFilesSelected = false;
		SaveLayoutDialogParams->LayoutFilePaths.Empty();
		SaveLayoutDialogParams->LayoutNames.Empty();
		SaveLayoutDialogParams->LayoutDescriptions.Empty();
	}
	
	CloseDialog();
	return FReply::Handled();
}

void SSaveLayoutDialog::CommitObjectPathForSave()
{
	// Update final values and close dialog if valid input and not duplicated
	if (bLastInputValidityCheckSuccessful)
	{
		// Sanity check
		if (SaveLayoutDialogParams.IsValid())
		{
			// Are you sure you want to do this?
			if (FPaths::FileExists(FinalFilePath) && !FSaveLayoutDialogUtils::OverrideLayoutDialog(FinalFilePath))
			{
				return;
			}
			// Update output parameters
			SaveLayoutDialogParams->bWereFilesSelected = bLastInputValidityCheckSuccessful;
			SaveLayoutDialogParams->LayoutFilePaths.Empty();
			SaveLayoutDialogParams->LayoutNames.Empty();
			SaveLayoutDialogParams->LayoutDescriptions.Empty();
			if (SaveLayoutDialogParams->bWereFilesSelected)
			{
				SaveLayoutDialogParams->LayoutFilePaths.Emplace(FinalFilePath);
				SaveLayoutDialogParams->LayoutNames.Emplace(CurrentlyEnteredLayoutName);
				SaveLayoutDialogParams->LayoutDescriptions.Emplace(CurrentlyEnteredLayoutDescription);
			}
		}
		else
		{
			ensureMsgf(false, TEXT("SaveLayoutDialogParams should never be a nullptr at this point, it is ideally initialized in the Construct() class."));
		}
		// Close dialog
		CloseDialog();
	}
}

void SSaveLayoutDialog::CloseDialog()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

bool SSaveLayoutDialog::IsValidFilePathForCreation(const FString& FilePath, const FString& FileName, FText& OutErrorMessage, bool bAllowExistingAsset) const
{
	// Make sure the name is not already a class or otherwise invalid for saving
	if (!FFileHelper::IsFilenameValidForSaving(FileName, OutErrorMessage))
	{
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Make sure the new name only contains valid characters
	if (!FName::IsValidXName(FileName, INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage))
	{
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Make sure we are not creating an FName that is too large
	if (FilePath.Len() >= NAME_SIZE)
	{
		// This asset already exists at this location, inform the user and continue
		OutErrorMessage = FText::Format(LOCTEXT("SaveLayoutNameTooLong", "This layout file name is too long ({0} characters), the maximum is {1}. Please choose a shorter name. File name: {2}"),
			FText::AsNumber(FilePath.Len()), FText::AsNumber(NAME_SIZE - 1), FText::FromString(FilePath));
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Make sure we are not creating an path that is too long for the OS
	const FString FullPath = FPaths::ConvertRelativePathToFull(FilePath);
	if (FullPath.Len() >= FPlatformMisc::GetMaxPathLength())
	{
		// The full path for the asset is too long
		OutErrorMessage = FText::Format(LOCTEXT("SaveLayoutFullPathTooLong", "The full path for the asset is too long ({0} characters), the maximum is {1}. Please choose a shorter name or create it in a shallower folder structure. Full path: {2}"),
			FText::AsNumber(FullPath.Len()), FText::AsNumber(FPlatformMisc::GetMaxPathLength() - 1), FText::FromString(FullPath));
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Check for an existing asset, unless it we were asked not to.
	if (!bAllowExistingAsset)
	{
		if (FPaths::FileExists(FilePath))
		{
			// This asset already exists at this location, inform the user and continue
			OutErrorMessage = FText::Format(LOCTEXT("SaveLayoutAlreadyExists", "An asset already exists at this location with the name '{0}'."), FText::FromString(FileName));

			// Return false to indicate that the user should enter a new name
			return false;
		}
	}

	return true;
}

void SSaveLayoutDialog::UpdateInputValidity()
{
	// Update file name
	FinalFileName = CurrentlyEnteredLayoutName.ToString();
	FSaveLayoutDialogUtils::SanitizeText(FinalFileName);
	// Fill full layout file path from file name and directory
	FinalFilePath =
		// Folder
		(DefaultDirectory.Right(1) == TEXT("/") ? DefaultDirectory : DefaultDirectory + TEXT("/"))
		// FileName + Extension
		+ (FinalFileName.Len() > SaveLayoutDialogParams->FileExtension.Len() && FinalFileName.Right(3) == SaveLayoutDialogParams->FileExtension
			? FinalFileName : FinalFileName + SaveLayoutDialogParams->FileExtension);

	// Validate file name
	bLastInputValidityCheckSuccessful = true;

	// No error text for an empty name. Just fail validity.
	if (FinalFileName.IsEmpty())
	{
		bLastInputValidityCheckSuccessful = false;
		LastInputValidityErrorText = FText::GetEmpty();
	}

	if (bLastInputValidityCheckSuccessful)
	{
		FText ErrorMessage;
		if (!IsValidFilePathForCreation(FinalFilePath, FinalFileName, ErrorMessage))
		{
			bLastInputValidityCheckSuccessful = false;
			LastInputValidityErrorText = ErrorMessage;
		}
	}
}

#undef LOCTEXT_NAMESPACE
