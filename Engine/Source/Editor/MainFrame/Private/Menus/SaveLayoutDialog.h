// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

/**
 * It saves the input parameters that FSaveLayoutDialogUtils::CreateSaveLayoutAsDialogInStandaloneWindow needs and the output ones that it will generate.
 */
class FSaveLayoutDialogParams
{
public:
	/**
	 * The constructor only contains the variables that FSaveLayoutDialogUtils::CreateSaveLayoutAsDialogInStandaloneWindow will need. The rest will be overriden.
	 */
	FSaveLayoutDialogParams(const FString& InDefaultDirectory, const FString& InFileExtension, const TArray<FName>& LayoutNames = TArray<FName>(),
		const TArray<FName>& LayoutDescriptions = TArray<FName>());

	/**
	 * The directory where the layout will be saved.
	 */
	const FString DefaultDirectory;

	/**
	 * The extension of the layout file.
	 */
	const FString FileExtension;

	/**
	 * Whether the user selected files (true) or cancelled/closed the window (false)
	 */
	bool bWereFilesSelected;

	/**
	 * The final full file path where the layout profile should be saved
	 */
	TArray<FString> LayoutFilePaths;

	/**
	 * The displayed layout name that will be visualized in Unreal. Its initial value will be used as default.
	 */
	TArray<FName> LayoutNames;

	/**
	 * The displayed layout description that will be visualized in Unreal. Its initial value will be used as default.
	 */
	TArray<FName> LayoutDescriptions;
};

/**
 * Some convenient UI functions regarding SWidgets for saving the layout
 */
class FSaveLayoutDialogUtils
{
public:
	/**
	 * Any special character (i.e., any non-alphanumeric character) will be turned into an underscore.
	 */
	static void SanitizeText(FString& InOutString);

	/**
	 * It will show a text asking whether you wanna override the existing UI Layout ini file.
	 * @param LayoutIniFileName The file path that might be overridden.
	 * @return True if the user agrees on overriding the file, false otherwise.
	 */
	static bool OverrideLayoutDialog(const FString& LayoutIniFileName);

	/**
	 * Self-contained function that creates a standalone SWindow, the SSaveLayoutDialog widget, and that blocks the thread until the dialog is finished
	 * @param InSaveLayoutDialogParams The FSaveLayoutDialogParams parameters used in the construction of SSaveLayoutDialog. Looking at its values:
	 *     1) DefaultDirectory and FileExtension will only be read.
	 *     2) bWereFilesSelected and LayoutFilePaths will only be written (i.e., their initial value will be overridden).
	 *     3) LayoutNames and LayoutDescriptions will only be written, but their initial values will also be used as default.
	 */
	static bool CreateSaveLayoutAsDialogInStandaloneWindow(const TSharedRef<FSaveLayoutDialogParams>& InSaveLayoutDialogParams);
};
