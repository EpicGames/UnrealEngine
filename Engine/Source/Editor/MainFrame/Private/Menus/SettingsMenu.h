// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISettingsModule.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsSection.h"

#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FSettingsMenu"

/**
 * Static helper class for populating the "Settings" menu.
 */
class FSettingsMenu
{
public:

	/**
	 * Creates the menu.
	 *
	 * @param MenuBuilder The builder for the menu that owns this menu.
	 * @param SettingsContainerName The name of the settings container to create the menu for.
	 */
	static void MakeMenu( UToolMenu* MenuBuilder, FName SettingsContainerName )
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule == nullptr)
		{
			return;
		}

		ISettingsContainerPtr SettingsContainer = SettingsModule->GetContainer(SettingsContainerName);

		if (!SettingsContainer.IsValid())
		{
			return;
		}

		TArray<ISettingsCategoryPtr> SettingsCategories;
			
		if (SettingsContainer->GetCategories(SettingsCategories) == 0)
		{
			return;
		}

		for (int32 CategoryIndex = 0; CategoryIndex < SettingsCategories.Num(); ++CategoryIndex)
		{
			ISettingsCategoryPtr SettingsCategory = SettingsCategories[CategoryIndex];
			TArray<ISettingsSectionPtr> SettingsSections;

			if (SettingsCategory->GetSections(SettingsSections) > 0)
			{
				FToolMenuSection& Section = MenuBuilder->AddSection(SettingsCategory->GetName(), SettingsCategory->GetDisplayName());

				SettingsSections.Sort(
					[](const ISettingsSectionPtr& First, const ISettingsSectionPtr& Second)
					{
						return First->GetDisplayName().ToString() < Second->GetDisplayName().ToString();
					});

				for (int32 SectionIndex = 0; SectionIndex < SettingsSections.Num(); ++SectionIndex)
				{
					ISettingsSectionPtr SettingsSection = SettingsSections[SectionIndex];

					Section.AddMenuEntry(
						SettingsSection->GetName(),
						SettingsSection->GetDisplayName(),
						SettingsSection->GetDescription(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateStatic(&FSettingsMenu::OpenSettings, SettingsContainerName, SettingsCategory->GetName(), SettingsSection->GetName()))
					);
				}
			}
		}
	}

	/**
	 * Opens the settings tab with the specified settings section.
	 *
	 * @param ContainerName The name of the settings container to open.
	 * @param CategoryName The name of the settings category that contains the section.
	 * @param SectionName The name of the settings section to select.
	 */
	static void OpenSettings( FName ContainerName, FName CategoryName, FName SectionName )
	{
		FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ContainerName, CategoryName, SectionName);
	}
};


#undef LOCTEXT_NAMESPACE
