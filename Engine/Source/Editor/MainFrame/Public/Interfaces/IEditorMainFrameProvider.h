// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWindow.h" // for EAutoCenter

class SWidget;

/** Set of configuration settings for the editor's main window. */
struct FMainFrameWindowOverrides
{
	FText WindowTitle;

	// If not set, will fallback to main Editor defaults
	TOptional<FVector2D> ScreenPosition;
	TOptional<FVector2D> WindowSize;
	TOptional<bool> bInitiallyMaximized;

	EAutoCenter CenterRules = EAutoCenter::None;

	bool bEmbedTitleAreaContent : 1;
	bool bIsUserSizable : 1;
	bool bSupportsMaximize : 1;
	bool bSupportsMinimize : 1;

	FMainFrameWindowOverrides()
		: bEmbedTitleAreaContent(true)
		, bIsUserSizable(true)
		, bSupportsMaximize(true)
		, bSupportsMinimize(true)
	{}
};

/** 
 * Base feature class, which serves as a hook for external sources to override 
 * the editor's main window on startup. 
 */
class IEditorMainFrameProvider : public IModularFeature
{
public:
	virtual ~IEditorMainFrameProvider() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("EditorMainFrameProvider"));
		return FeatureName;
	}

	/**
	 * Signals if this provider wishes to override the editor's main frame on 
	 * editor startup.
	 * NOTE: When true, this will block the standard editor window from appearing 
	 *       (instead replacing it with the provider's widget). 
	 */
	virtual bool IsRequestingMainFrameControl() const = 0;

	/** 
	 * Specifies the desired window title, dimensions, etc. -- for the main frame 
	 * window (to fit the provier's widget).
	 */
	virtual FMainFrameWindowOverrides GetDesiredWindowConfiguration() const = 0;

	/**
	 * Spawns a widget for slotting into the main editor window (in-place of the 
	 * standard editor).
	 * @return A replacement widget for the editor's main window.
	 */
	virtual TSharedRef<SWidget> CreateMainFrameContentWidget() const = 0;
};
