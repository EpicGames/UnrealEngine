// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Settings/HomeScreenCommon.h"

/** 
 * Base feature class, which serves as a hook for external sources to override 
 * the editor's HomeScreen window content.
 */
class IHomeScreenProvider : public IModularFeature
{
public:
	virtual ~IHomeScreenProvider() {}

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("HomeScreenProvider"));
		return FeatureName;
	}

	/**
	 * @return True if we want our HomeScreen settings to be used for the HomeScreen widget inside the HomeScreen tab
	 */
	virtual bool IsRequestingHomeScreenPageControl() const = 0;

	/**
	 * Less priority means that it will be ignored for higher ones.
	 * The editor default HomeScreen priority is -1
	 * 
	 * @return The priority for this HomeScreen
	 */
	virtual int32 GetHomeScreenPriority() = 0;

	/**
	 * @return Settings to use for this HomeScreen
	 */
	virtual FHomeScreenWidgetSettings GetHomeScreenWidgetSettings() const = 0;
};
