// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Settings/HomeScreenCommon.h"

class FTabManager;
class SWindow;
struct FToolMenuContext;

/**
 * Data needed to display a Developer tools in the status bar.
 */
struct FMainFrameDeveloperTool
{
	/* Visiblility of the developer tool. */
	TAttribute<EVisibility> Visibility;
	/* Label of the developer tool. */
	TAttribute<FText> Label;
	/* Value of the developer tool. */
	TAttribute<FText> Value;
};

using FShowProjectBrowser = TDelegate<void(bool bAllowProjectOpening, bool bAllowProjectCreate)>;

/**
 * Interface for main frame modules.
 */
class IMainFrameModule
	: public IModuleInterface
{
public:

	/**
	 * Create the HomeScreen widget, will use the default settings if no settings are provided and return the newly created HomeScreen
	 * Should only be used when using the IEditorMainFrameProvider if you wish to use a HomeScreen widget with different settings
	 * 
	 * @param InParentTab OwnerTab of the HomeScreen, needed when the HomeScreen is created as a Tab to avoid MAC to render the WebPage when switching tabs
	 * @param InSettings HomeScreenSettings to use to create the current HomeScreen
	 * @return Return the newly created HomeScreen widget
	 */
	virtual TSharedRef<SWidget> CreateHomeScreenWidget(const TSharedPtr<SDockTab>& InParentTab, FHomeScreenWidgetSettings InSettings = FHomeScreenWidgetSettings()) = 0;

	/**
	 * Creates the default editor main frame
	 *
	 * @param bStartImmersive True to force a main frame viewport into immersive mode
	 * @param bStartPIE True to start a PIE session right away
	 */
	virtual void CreateDefaultMainFrame( const bool bStartImmersive, const bool bStartPIE ) = 0;

	/**
	 * Recreates the default editor main frame.
	 * I.e., if CreateDefaultMainFrame or RecreateDefaultMainFrame were already called, it would clean the previous default main frame and create it again.
	 *
	 * @param bStartImmersive True to force a main frame viewport into immersive mode
	 * @param bStartPIE True to start a PIE session right away
	 */
	virtual void RecreateDefaultMainFrame(const bool bStartImmersive, const bool bStartPIE) = 0;

	/**
	 * Recreates the default editor main frame content without destroying the old one.
	 *
	 * @param bStartImmersive True to force a main frame viewport into immersive mode
	 * @param bStartPIE True to start a PIE session right away
	 */
	virtual void ReconstructDefaultMainFrameContent(const bool bStartImmersive, const bool bStartPIE) = 0;

	/**
	 * Returns true if the Default Main Frame is being recreated. 
	 */
	virtual bool IsRecreatingDefaultMainFrame() const = 0;

	/**
	 * Generates a menu that includes application global commands, such as "Save All", "Exit", etc.  If you're building
	 * a menu for your tab, you should call this function to create your menu, passing in an extender object to add your
	 * tab-specific menu items!
	 *
	 * @param	TabManager	The tab manager for the tab you're creating the menu for.  This is needed so we can populate the layout menus correctly.
	 * @param	Extender	Extender object used to customize the main frame menu
	 *
	 * @return	The newly-created menu widget
	 */
	virtual TSharedRef<SWidget> MakeMainMenu( const TSharedPtr<FTabManager>& TabManager, const FName MenuName, FToolMenuContext& ToolMenuContext ) const = 0;

	/**
	 * Generates a menu that's just like the "main menu" widget above, except it also includes some infrequently used commands
	 * that are better off only shown in a single "main tab" within the application
	 *
	 * @param	TabManager	The tab manager for the tab you're creating the menu for.  This is needed so we can populate the layout menus correctly.
	 * @param	Extender	Extender object used to customize the main frame menu
	 *
	 * @return	The newly-created menu widget
	 */

	/**
	 * Generates a menu for status and developer
	 *
	 * @param	AdditionalTools	Additional developer tools that would be added to the main frame menu
	 *
	 * @return	The newly-created menu widget
	 */
	virtual TSharedRef<SWidget> MakeDeveloperTools( const TArray<FMainFrameDeveloperTool>& AdditionalTools ) const = 0;

	/**
	 * Checks to see if the main frame window is currently initialized
	 *
	 * @return	True if initialized, otherwise false
	 */
	virtual bool IsWindowInitialized() const = 0;

	/**
	 * Gets the window the mainframe lives in.
	 *
	 * @return The window widget.
	 */
	virtual TSharedPtr<SWindow> GetParentWindow( ) const = 0;

	/**
	 * Sets the reference to the main tab.
	 *
	 * @param MainTab - The main tab.
	 */
	virtual void SetMainTab( const TSharedRef<SDockTab>& MainTab ) = 0;

	/**
	 * Enables the delegate responsible for shutting down the editor when the main tab is closed.
	 */
	virtual void EnableTabClosedDelegate( ) = 0;

	/**
	 * Disables the delegate responsible for shutting down the editor when the main tab is closed.
	 */
	virtual void DisableTabClosedDelegate( ) = 0;

	/**
	 * Requests that the editor be closed
	 * In some cases the editor may not be closed (like if a user cancels a save dialog)
	 */
	virtual void RequestCloseEditor( ) = 0;

	/**
	 * Updates the mainframe title on the Slate window and the native OS window underneath
	 *
	 * @param InLevelFileName  Full level filename from which the base name will be stripped and used to make the window title
	 */
	virtual void SetLevelNameForWindowTitle(const FString& InLevelFileName) = 0;

	/**
	 * Overrides the title of the application that's displayed in the title bar area and other locations
	 *
	 * @param	NewOverriddenApplicationTitle	The text to be displayed in the window title, or empty to use the application's default text
	 */
	virtual void SetApplicationTitleOverride(const FText& NewOverriddenApplicationTitle) = 0;

	/**
	 * Returns a friendly string name for the currently loaded persistent level.
	 *
	 * @return Name of the loaded level.
	 */
	virtual FString GetLoadedLevelName() const = 0;

	virtual TSharedRef<FUICommandList>& GetMainFrameCommandBindings( ) = 0;

	/**
	 * Gets the MRU/Favorites list
	 *
	 * @return	MRU/Favorites list
	 */
	virtual class FMainMRUFavoritesList* GetMRUFavoritesList() const = 0;

	/**
	 * Gets the title string for the application, optionally including the current game name as part of the title
	 *
	 * @param	bIncludeGameName	True if the game name should be included as part of the returned title string
	 *
	 * @return	Returns the title of the application, to be displayed in tabs or window titles 
	 */
	virtual const FText GetApplicationTitle( const bool bIncludeGameName ) const = 0;

	/**
	 * Shows the 'About UnrealEd' window.
	 */
	virtual void ShowAboutWindow( ) const = 0;

	/**
	 * Delegate for binding functions to be called when the mainframe finishes up getting created.
	 */
	DECLARE_EVENT_TwoParams(IMainFrameModule, FMainFrameCreationFinishedEvent, TSharedPtr<SWindow>, bool);
	virtual FMainFrameCreationFinishedEvent& OnMainFrameCreationFinished( ) = 0;

	/**
	 * Delegate for when a platform SDK isn't installed corrected (takes the platform name and the documentation link to show)
	 */
	DECLARE_EVENT_TwoParams(IMainFrameModule, FMainFrameSDKNotInstalled, const FString&, const FString&);
	virtual FMainFrameSDKNotInstalled& OnMainFrameSDKNotInstalled( ) = 0;
	virtual void BroadcastMainFrameSDKNotInstalled(const FString& PlatformName, const FString& DocLink) = 0;

	/**
	 * Delegate for making an open-ended request for a resource or link.
	 */
	DECLARE_EVENT_TwoParams(IMainFrameModule, FMainFrameRequestResource, const FString&, const FString&);
	virtual FMainFrameRequestResource& OnMainFrameRequestResource() = 0;
	virtual void BroadcastMainFrameRequestResource(const FString& Category, const FString& ResourceName) = 0;

	/**
	 * Enable external control of when main frame is shown
	 */
	virtual void EnableDelayedShowMainFrame() = 0;

	/**
	 * Show main frame now if it was delayed and not shown yet
	 */
	virtual void ShowDelayedMainFrame() = 0;

	/**
	 * Delegate for determining if the editor can be closed
	 */
	DECLARE_DELEGATE_RetVal(bool, FMainFrameCanCloseEditor);

	/**
	 * Register a new callback for determining if the editor can be closed
	 */
	virtual FDelegateHandle RegisterCanCloseEditor(const FMainFrameCanCloseEditor& InDelegate) = 0;

	/**
	 * Unregister a callback for determining if the editor can be closed
	 */
	virtual void UnregisterCanCloseEditor(FDelegateHandle InHandle) = 0;

	virtual bool ExecuteCanCloseEditorDelegates() = 0;

	/**
	 * Overrides the section that gets selected by default when opening editor settings
	 * @note Call with default parameters to clear the override
	 */
	virtual void SetEditorSettingsDefaultSelectionOverride(FName CategoryName = FName(), FName SectionName = FName()) = 0;

	/** 
	 * Gets the override for the section that gets selected by default when opening editor settings
	 * @note Returns None if there's no override
	 */
	virtual void GetEditorSettingsDefaultSelectionOverride(FName& OutCategoryName, FName& OutSectionName) = 0;

	/**
	 * Opens the project browser in a new window.
	 */
	virtual void ShowProjectBrowser(bool bAllowProjectOpening, bool bAllowProjectCreate) = 0;

	/**
	 * Overrides the functionality of ShowProjectBrowser with the provided delegate.
	 */
	virtual FDelegateHandle RegisterShowProjectBrowserOverride(const FShowProjectBrowser& InShowProjectBrowserOverride) = 0;

	/**
	 * Removes the override of ShowProjectBrowser if it matches the handle.
	 */
	virtual void UnregisterShowProjectBrowserOverride(FDelegateHandle InHandle) = 0;

	/**
	 * Return the HomeScreen tab label
	 */
	static FText& GetHomeScreenTabLabel()
	{
		static FText HomeScreenTabLabel = NSLOCTEXT("MainFrameModule", "HomePanelDisplayName", "Home Panel");
		return HomeScreenTabLabel;
	}

public:

	/**
	 * Gets a reference to the search module instance.
	 *
	 * @todo gmp: better implementation using dependency injection.
	 * @return A reference to the MainFrame module.
	 */
	static IMainFrameModule& Get( )
	{
		static const FName MainFrameModuleName = "MainFrame";
		return FModuleManager::LoadModuleChecked<IMainFrameModule>(MainFrameModuleName);
	}

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~IMainFrameModule( ) = default;
};
