// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public Dependencies
 *****************************************************************************/

#include "Modules/ModuleInterface.h"
#include "SlateBasics.h"


/* Public Includes
 *****************************************************************************/

#include "Interfaces/IMainFrameModule.h"
