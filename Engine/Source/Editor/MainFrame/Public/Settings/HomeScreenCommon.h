// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "HomeScreenCommon.generated.h"

UENUM()
enum class EMainSectionMenu : uint8
{
	None,
	Home,
	News,
	GettingStarted,
	SampleProjects
};

UENUM()
enum class EAutoLoadProject : uint8
{
	HomeScreen UMETA(DisplayName = "Home Panel"),
	LastProject UMETA(DisplayName = "Most Recent Project"),
	MAX UMETA(Hidden)
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoadAtStartupChanged, EAutoLoadProject)
DECLARE_DELEGATE_OneParam(FOnSetLoadAtStartup, bool)
DECLARE_DELEGATE_RetVal(bool, FOnGetLoadAtStartup)

/**
 * Struct containing information on the MainSections in the HomeScreen
 */
struct FHomeScreenSectionLink
{
	FString BeforeLocaleURL;
	FString AfterLocaleURL;
	FString AfterLocaleNoProjectURL;
};

using FHomeScreenSectionToLinkMap = TMap<EMainSectionMenu, FHomeScreenSectionLink>;

/**
 * Struct containing information of ResourceAndCommunity section entries in the HomeScreen
 */
struct FHomeScreenResourceCommunityEntry
{
	TAttribute<FString> Link;
	FText DisplayName;
	FName IconBrushName;
};

struct FHomeScreenLoadAtStartupSetting
{
public:
	FHomeScreenLoadAtStartupSetting() = default;
	FHomeScreenLoadAtStartupSetting(FOnSetLoadAtStartup InOnSetLoadAtStartup, FOnGetLoadAtStartup InOnGetLoadAtStartup)
		: OnSetLoadAtStartupDelegate(InOnSetLoadAtStartup)
		, OnGetLoadAtStartupDelegate(InOnGetLoadAtStartup)
	{
	}

public:
	FOnLoadAtStartupChanged& OnLoadAtStartupChanged() { return OnLoadAtStartupChangedDelegate; }
	void SetOnLoadAtStartup(bool InAutoLoadOption) const { OnSetLoadAtStartupDelegate.ExecuteIfBound(InAutoLoadOption); }
	bool GetOnLoadAtStartup() const { return OnGetLoadAtStartupDelegate.IsBound() ? OnGetLoadAtStartupDelegate.Execute() : false; }

private:
	FOnLoadAtStartupChanged OnLoadAtStartupChangedDelegate;
	FOnSetLoadAtStartup OnSetLoadAtStartupDelegate;
	FOnGetLoadAtStartup OnGetLoadAtStartupDelegate;
};

DECLARE_DELEGATE_RetVal(FHomeScreenLoadAtStartupSetting&, FOnGetStartupSetting)

/** Setting used to create the HomeScreen, if leaved empty the defaults are used instead */
struct FHomeScreenWidgetSettings
{
	FHomeScreenWidgetSettings()
		: bAlwaysShowRecentList(false)
		, RecentProjectListWidget(SNullWidget::NullWidget)
		, UnrealLogoBrush(nullptr)
		, OnGetStartupSetting()
	{
	}

	/** 
	 * Section x Link connection, if leaved empty it will use the UE default section x links.
	 * If at least 1 is populated it will use this instead, every missing section are skipped from being created in this case
	 */
	FHomeScreenSectionToLinkMap SectionLinks;

	/**
	 * Additional Resource entry to add to the default ones
	 */
	TArray<FHomeScreenResourceCommunityEntry> ResourcesEntries;

	/**
	 * Whether the recent project list should always be displayed without checking for their presence.
	 */
	bool bAlwaysShowRecentList;

	/**
	 * Widget displaying the recent project list.
	 */
	TSharedRef<SWidget> RecentProjectListWidget;

	/**
	 * Brush representing the icon for the Unreal Logo.
	 */
	const FSlateBrush* UnrealLogoBrush;

	/**
	 * Used to grab the StartupSetting logic to use
	 */
	FOnGetStartupSetting OnGetStartupSetting;
};
