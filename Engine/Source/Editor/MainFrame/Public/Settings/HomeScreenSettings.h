// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HomeScreenCommon.h"
#include "UObject/Object.h"
#include "HomeScreenSettings.generated.h"

// Load on Startup setting is registered with the FEditorLoadingSavingSettingsCustomization in the Loading & Saving Section

UCLASS(MinimalAPI, config=EditorSettings)
class UHomeScreenSettings : public UObject
{
	GENERATED_BODY()

public:
	UHomeScreenSettings();

public:
	UPROPERTY()
	EAutoLoadProject LoadAtStartup = EAutoLoadProject::HomeScreen;

public:
	MAINFRAME_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	MAINFRAME_API virtual void PostInitProperties() override;

	UE_DEPRECATED(5.8, "Use the GetLoadAtStartupSetting instead")
	FOnLoadAtStartupChanged& OnLoadAtStartupChanged() { return OnLoadAtStartupChangedDelegate; }

	MAINFRAME_API FHomeScreenLoadAtStartupSetting& GetLoadAtStartupSetting();

private:
	void OnMostRecentProjectEditorSettingChanged(bool bInAutoLoad);
	void SetLoadAtStartupChanged(bool bInAutoLoad);
	bool GetLoadAtStartupChanged() const;

private:
	FHomeScreenLoadAtStartupSetting HomeScreenLoadAtStartupSetting;
	FOnLoadAtStartupChanged OnLoadAtStartupChangedDelegate;
};
